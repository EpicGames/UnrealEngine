// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Http/HttpClient.h"
#include "Serialization/CompactBinaryPackage.h"
#include "ZenCbPackageReceiver.h"

namespace UE::Zen
{

class FZenServiceInstance;

class FAsyncCbPackageReceiver : public IHttpReceiver
{
public:
	using FOnComplete = TUniqueFunction<void(FAsyncCbPackageReceiver* Receiver)>;

	FAsyncCbPackageReceiver(const FAsyncCbPackageReceiver&) = delete;
	FAsyncCbPackageReceiver& operator=(const FAsyncCbPackageReceiver&) = delete;

	ZEN_API FAsyncCbPackageReceiver(
		THttpUniquePtr<IHttpRequest>&& InRequest,
		FZenServiceInstance& InZenServiceInstance,
		FOnComplete&& InOnComplete,
		int InMaxAttempts,
		bool InUseSoftRecovery);

	ZEN_API const IHttpResponse& GetHttpResponse();
	ZEN_API const FCbPackage& GetResponsePackage();

	ZEN_API void SendAsync();

	ZEN_API FString GetPayloadAsString() const;
protected:
	// IHttpReceiver Interface

	ZEN_API IHttpReceiver* OnCreate(IHttpResponse& LocalResponse);
	ZEN_API IHttpReceiver* OnComplete(IHttpResponse& LocalResponse);

protected:
	THttpUniquePtr<IHttpRequest> Request;
	THttpUniquePtr<IHttpResponse> Response;
	FZenServiceInstance& ZenServiceInstance;
	FCbPackage Package;
	UE::Zen::FCbPackageReceiver BaseReceiver;
	FOnComplete OnCompleteCallback;
	const int MaxAttempts;
	const bool UseSoftRecovery;
	int Attempt;
};

} // namespace UE::Zen
