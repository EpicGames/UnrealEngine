// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "HAL/UnrealMemory.h"
#include "Memory/CompositeBuffer.h"
#include "Memory/SharedBuffer.h"
#include "Misc/AssertionMacros.h"

namespace UE::Zen {

/** Copy Size bytes from Source at offset SourceOffset into Dest at offset 0. */
inline void Memcpy(void* Dest, const FCompositeBuffer& Source, size_t SourceOffset, size_t Size)
{
	uint64 Offset = 0;
	uint64 EndOffset = SourceOffset + Size;
	for (const FSharedBuffer& Segment : Source.GetSegments())
	{
		uint64 SegmentEnd = Offset + Segment.GetSize();
		bool bDone = false;
		if (SegmentEnd > EndOffset)
		{
			bDone = true;
			SegmentEnd = EndOffset;
		}
		if (Offset < SourceOffset)
		{
			if (SegmentEnd >= SourceOffset)
			{
				const uint8* SegData = reinterpret_cast<const uint8*>(Segment.GetData()) + (SourceOffset - Offset);
				uint64 CopySize = SegmentEnd - SourceOffset;
				FMemory::Memcpy(Dest, SegData, CopySize);
				Dest = static_cast<uint8*>(Dest) + CopySize;
			}
		}
		else
		{
			uint64 CopySize = SegmentEnd - Offset;
			FMemory::Memcpy(Dest, Segment.GetData(), CopySize);
			Dest = static_cast<uint8*>(Dest) + CopySize;
		}
		Offset = SegmentEnd;
		if (bDone)
		{
			break;
		}
	}
	check(Offset == EndOffset);
}

} // namespace UE::Zen
