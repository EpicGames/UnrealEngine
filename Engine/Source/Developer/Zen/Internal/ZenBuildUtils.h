// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Experimental/BuildServerInterface.h"

#define UE_API ZEN_API

namespace UE::Zen::Build
{

struct FBuildState
{
	FString Namespace;
	TArray<UE::Zen::Build::FBuildServiceInstance::FBuildRecord> Results;
};

struct FListBuildsState
{
	std::atomic<uint32> PendingQueries;
	TArray<FBuildState> QueryState;
};

struct FBuildReference
{
	FString Namespace;
	FString Bucket;
	FCbObjectId BuildId;
};

UE_API bool TryParseBuildReferenceFromUrl(FStringView Url, FBuildReference& OutReference);

} // namespace UE::Zen::Build

#undef UE_API
