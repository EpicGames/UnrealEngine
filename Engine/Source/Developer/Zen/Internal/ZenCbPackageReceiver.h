// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Http/HttpClient.h"

class FCbPackage;

namespace UE::Zen
{

class FZenServiceInstance;

class FCbPackageReceiver final : public IHttpReceiver
{
public:
	FCbPackageReceiver(const FCbPackageReceiver&) = delete;
	FCbPackageReceiver& operator=(const FCbPackageReceiver&) = delete;

	ZEN_API explicit FCbPackageReceiver(FCbPackage& OutPackage, IHttpReceiver* InNext = nullptr);

	ZEN_API void Reset();
	ZEN_API const FMemoryView Body() const;

	ZEN_API bool ShouldRetry(FZenServiceInstance& ZenServiceInstance, IHttpResponse& LocalResponse, bool UseSoftRecovery) const;

	ZEN_API static FString GetPayloadAsString(EHttpMediaType ContentType, FMemoryView BodyView);

private:
	ZEN_API IHttpReceiver* OnCreate(IHttpResponse& Response) final;
	ZEN_API IHttpReceiver* OnComplete(IHttpResponse& Response) final;

	bool TryRecover(Zen::FZenServiceInstance& ZenServiceInstance, bool UseSoftRecovery) const;

private:
	FCbPackage& Package;
	IHttpReceiver* Next;
	TArray64<uint8> BodyArray;
	FHttpByteArrayReceiver BodyReceiver{BodyArray, this};
};

} // namespace UE::Zen
