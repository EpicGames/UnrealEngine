// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/CompactBinaryPackage.h"
#include "Templates/Function.h"

namespace UE::Zen {

/**
 * The stream response status.
 */
enum class EZenStreamStatus
{
	Ok,
	Error,
	Canceled,
	Completed
};

/**
 * Stream response.
 */
struct FZenStreamResponse
{
	FCbPackage Response;
	EZenStreamStatus Status = EZenStreamStatus::Error;
};

/** Stream response callback. */
using FOnStreamResponse = TUniqueFunction<void(FZenStreamResponse&& Response)>;

/**
 * Zen server web socket client.
 */
class IZenClient
{
public:
	virtual ~IZenClient() = default;
	
	virtual bool Connect(FStringView Host, int32 Port) = 0;
	virtual void Disconnect() = 0;
	virtual bool IsConnected() const = 0;

	virtual bool SendRequest(FCbPackage&& Request, FOnStreamResponse&& OnResponse) = 0;
	virtual bool SendRequest(FCbObject&& Request, FOnStreamResponse&& OnResponse) = 0;
	virtual bool SendStreamRequest(FCbObject&& Request, FOnStreamResponse&& OnStreamResponse) = 0;

	ZEN_API static TUniquePtr<IZenClient> Create();
};

/**
 * Zen server client pool.
 */
class IZenClientPool
{
public:
	virtual ~IZenClientPool() = default;

	virtual bool Connect(FStringView Host, int32 Port, int32 PoolSize = 8) = 0;
	virtual void Disconnect() = 0;
	virtual bool IsConnected() const = 0;

	virtual bool SendRequest(FCbPackage&& Request, FOnStreamResponse&& OnResponse) = 0;
	virtual bool SendRequest(FCbObject&& Request, FOnStreamResponse&& OnResponse) = 0;
	virtual bool SendStreamRequest(FCbObject&& Request, FOnStreamResponse&& OnStreamResponse) = 0;
	
	ZEN_API static TUniquePtr<IZenClientPool> Create();
};

} // namespace UE::Zen
