// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Experimental/ZenServerInterface.h"
#include "Memory/SharedBuffer.h"
#include "Serialization/CompactBinary.h"

class FArchive;
class FCbAttachment;
class FCbPackage;
class FCbWriter;

namespace UE::Zen {

namespace OpLog
{
ZEN_API void SaveCbAttachment(const FCbAttachment& Attachment, FCbWriter& Writer);
ZEN_API void SaveCbPackage(const FCbPackage& Package, FCbWriter& Writer);
ZEN_API void SaveCbPackage(const FCbPackage& Package, FArchive& Ar);
ZEN_API bool TryLoadCbPackage(FCbPackage& Package, FArchive& Ar, FCbBufferAllocator Allocator = FUniqueBuffer::Alloc);
}

namespace Http
{
static const uint32 kCbPkgMagic = 0xaa77aacc;
	
enum class RpcAcceptOptions : uint32_t
{
	kNone = 0,
	kAllowLocalReferences = (1u << 0),
	kAllowPartialCacheChunks = (1u << 2)
};

ZEN_API void SaveCbPackage(const FCbPackage& Package, FArchive& Ar);
ZEN_API bool TryLoadCbPackage(FCbPackage& Package, FArchive& Ar, FCbBufferAllocator Allocator = FUniqueBuffer::Alloc);
}

} // namespace UE::Zen
