// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StaticArray.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Memory/MemoryFwd.h"

// if there is a platform-specific include then it must be used in the header file in case it defines CURL_STRICTER
#if defined(PLATFORM_CURL_INCLUDE)

#if PLATFORM_MICROSOFT
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#endif

#include PLATFORM_CURL_INCLUDE

#if PLATFORM_MICROSOFT
#include "Microsoft/HideMicrosoftPlatformTypes.h"
#endif

#endif //defined(PLATFORM_CURL_INCLUDE)


class FCompositeBuffer;
class FCbObjectView;
class FCbPackage;

namespace UE::Zen
{

class FZenServiceEndpoint;

static bool IsSuccessCode(int ResponseCode)
{
	return 200 <= ResponseCode && ResponseCode < 300;
}

enum class EContentType
{
	Binary				= 0,
	Text				= 1,
	JSON				= 2,
	CbObject			= 3,
	CbPackage			= 4,
	YAML				= 5,
	CbPackageOffer		= 6,
	CompressedBinary	= 7,
	UnknownContentType	= 8,
	Count
};

inline FStringView GetMimeType(EContentType Type)
{
	switch (Type)
	{
		case EContentType::Binary:
			return TEXTVIEW("application/octet-stream");
		case EContentType::Text:
			return TEXTVIEW("text/plain");
		case EContentType::JSON:
			return TEXTVIEW("application/json");
		case EContentType::CbObject:
			return TEXTVIEW("application/x-ue-cb");
		case EContentType::CbPackage:
			return TEXTVIEW("application/x-ue-cbpkg");
		case EContentType::YAML:
			return TEXTVIEW("text/yaml");
		case EContentType::CbPackageOffer:
			return TEXTVIEW("application/x-ue-offer");
		case EContentType::CompressedBinary:
			return TEXTVIEW("application/x-ue-comp");
		default:
			return TEXTVIEW("unknown");
	}
}

/** Minimal HTTP request type wrapping CURL without the need for managers. This request
  * is written to allow reuse of request objects, in order to allow connections to be reused.
	
  * CURL has a global library initialization (curl_global_init). We rely on this happening in
  * the Online/HTTP library which is a dependency of this module
  */

class FZenHttpRequest
{
public:
	ZEN_API FZenHttpRequest(const FZenServiceEndpoint& InDomain, bool bInLogErrors, uint32 InConnectTimeoutMS = 0, uint32 InTimeoutMS = 0);
	ZEN_API ~FZenHttpRequest();

	/**
	  * Resets all options on the request except those that should always be set.
	  */
	ZEN_API void Reset();

	/**
	 * Initializes a previously-allocated FZenHttpRequest with the options that can vary between requests
	 */
	ZEN_API void Initialize(bool bInLogErrors);

	/** Returns the HTTP response code.*/
	inline const int GetResponseCode() const
	{
		return int(ResponseCode);
	}

	inline const int GetErrorCode() const
	{
		return int(CurlResult);
	}

	ZEN_API FAnsiStringView GetError() const;

	ZEN_API FString GetDetailedResultString(FStringView Uri) const;

	inline const bool GetResponseFormatValid() const
	{
		return bResponseFormatValid;
	}

	/** Returns the number of bytes sent during this request (headers withstanding). */
	inline const size_t  GetBytesSent() const
	{
		return BytesSent;
	}

	/**
		* Convenience result type interpreted from HTTP response code.
		*/
	enum class Result
	{
		Success,
		Failed
	};

	/**
		* Upload buffer using the request, using PUT verb
		* @param Uri Url to use.
		* @param Buffer Data to upload
		* @param ContentType The content MIME type.
		* @return Result of the request
		*/
	ZEN_API Result PerformBlockingPut(const TCHAR* Uri, const FCompositeBuffer& Buffer, EContentType ContentType);

	/**
	* Download an url into a buffer using the request.
	* @param Uri Url to use.
	* @param Buffer Optional buffer where data should be downloaded to. If this is null then
	* downloaded data will be stored in an internal buffer and accessed via GetResponseAsString
	* @param ContentType The MIME type to accept.
	* @return Result of the request
	*/
	ZEN_API Result PerformBlockingDownload(FStringView Uri, TArray64<uint8>* Buffer, EContentType AcceptType);

	/**
		* Download an url into a buffer using the request.
		* @param Uri Url to use.
		* @param OutPackage Package instance which will receive the data
		* @result Request success/failure status
		*/
	ZEN_API Result PerformBlockingDownload(const TCHAR* Uri, FCbPackage& OutPackage);

	/**
		* Query an url using the request. Queries can use either "Head" or "Delete" verbs.
		* @param Uri Url to use.
		* @param ContentType The MIME type to accept.
		* @return Result of the request
		*/
	ZEN_API Result PerformBlockingHead(FStringView Uri, EContentType AcceptType);

	/**
		* Query an url using the request. Queries can use either "Head" or "Delete" verbs.
		* @param Uri Url to use.
		* @return Result of the request
		*/
	ZEN_API Result PerformBlockingDelete(FStringView Uri);

	ZEN_API Result PerformBlockingPostPackage(FStringView Uri, const FCbPackage& Package,
		EContentType AcceptType = EContentType::UnknownContentType);
	ZEN_API Result PerformBlockingPost(FStringView Uri, FCbObjectView Obj,
		EContentType AcceptType = EContentType::UnknownContentType);
	ZEN_API Result PerformBlockingPost(FStringView Uri, FMemoryView Payload,
		EContentType ContentType = EContentType::Binary, EContentType AcceptType = EContentType::UnknownContentType);

	ZEN_API Result PerformRpc(FStringView Uri, FCbObjectView Request, FCbPackage &OutResponse);
	ZEN_API Result PerformRpc(FStringView Uri, const FCbPackage& Request, FCbPackage& OutResponse);

	/** Returns the response buffer as a string. Note that if the request is performed
		with an external buffer as target buffer this string will be empty.
	 */
	ZEN_API FString GetResponseAsString() const;

	/** Returns the response buffer. Note that if the request is performed
	  * with an external buffer as target buffer this will be empty.
	  */
	inline const TArray64<uint8>& GetResponseBuffer() const
	{
		return ResponseBuffer;
	}

	ZEN_API FCbObjectView GetResponseAsObject() const;
	ZEN_API FCbPackage GetResponseAsPackage() const;

	ZEN_API bool GetHeader(const ANSICHAR* Header, FString& OutValue) const;

	ZEN_API static FAnsiStringView LexResponseCodeToString(int HttpCode);

private:
#if defined(CURL_STRICTER)
	CURL*					Curl = nullptr;
#else
	void* /* CURL */		Curl = nullptr;
#endif
	long /* CURLCode */		CurlResult;
	long					ResponseCode = 0;
	size_t					BytesSent = 0;
	size_t					BytesReceived = 0;
	bool					bLogErrors = false;
	bool					bResponseFormatValid = false;
	uint32				ConnectTimeoutMS = 0;
	uint32				TimeoutMS = 0;

	const FCompositeBuffer*	ReadDataView = nullptr;
	TArray64<uint8>*		WriteDataBufferPtr = nullptr;
	FCbPackage*				WriteDataPackage = nullptr;
	TArray64<uint8>*		WriteHeaderBufferPtr = nullptr;

	TArray64<uint8>			ResponseHeader;
	TArray64<uint8>			ResponseBuffer;	// If no other response buffer is set, this is where the response payload goes
	TArray<FString>			Headers;
	FString					Domain;
	const FZenServiceEndpoint& Endpoint;

	void AddHeader(FStringView Header, FStringView Value);

	/**
	  * Supported request verb
	  */
	enum class RequestVerb
	{
		Get,
		Put,
		Post,
		Delete,
		Head
	};

	void LogResult(long /*CURLcode*/ Result, const TCHAR* Uri, RequestVerb Verb) const;

	/**
		* Performs the request, blocking until finished.
		* @param Uri Address on the domain to query
		* @param Verb HTTP verb to use
		* @param Buffer Optional buffer to directly receive the result of the request.
		* If unset the response body will be stored in the request.
		*/
	Result PerformBlocking(FStringView Uri, RequestVerb Verb, uint64 ContentLength);

	FZenHttpRequest::Result ParseRpcResponse(FZenHttpRequest::Result ResultFromPost, FCbPackage& OutResponse);

	static FString GetAnsiBufferAsString(const TArray64<uint8>& Buffer);

	struct FStatics;
};

/**
  * Pool which manages a fixed set of requests. Users are required to release requests that have been
  * acquired. 
  * 
  * Intended to be used with \ref FScopedRequestPtr which handles lifetime management transparently
  */
struct FZenHttpRequestPool
{
	ZEN_API explicit FZenHttpRequestPool(const FZenServiceEndpoint& Endpoint, uint32 PoolEntryCount = 16);
	ZEN_API ~FZenHttpRequestPool();

	/** Block until a request is free. Once a request has been returned it is
	  * "owned by the caller and need to release it to the pool when work has been completed.
	  * @return Usable request instance.
	  */
	ZEN_API FZenHttpRequest* WaitForFreeRequest();

	/** Release request to the pool.
	  * @param Request Request that should be freed. Note that any buffer owned by the request can now be reset.
	  */
	ZEN_API void ReleaseRequestToPool(FZenHttpRequest* Request);

private:
	struct FEntry
	{
		std::atomic<uint8> IsAllocated;
		FZenHttpRequest* Request;
	};

	TArray<FEntry> Pool;
};

/**
  * Utility class to manage requesting and releasing requests from the \ref FRequestPool.
  */
struct FZenScopedRequestPtr
{
public:
	FZenScopedRequestPtr(FZenHttpRequestPool* InPool, bool bLogErrors=true)
	:	Request(InPool->WaitForFreeRequest())
	,	Pool(InPool)
	{
		Request->Initialize(bLogErrors);
	}

	~FZenScopedRequestPtr()
	{
		Pool->ReleaseRequestToPool(Request);
	}

	inline bool IsValid() const
	{
		return Request != nullptr;
	}

	inline operator bool() const { return IsValid(); }

	FZenHttpRequest* operator->()
	{
		check(IsValid());
		return Request;
	}

	FZenHttpRequest& operator*()
	{
		check(IsValid());
		return *Request;
	}

private:
	FZenHttpRequest*		Request;
	FZenHttpRequestPool*	Pool;
};

} // namespace UE::Zen
