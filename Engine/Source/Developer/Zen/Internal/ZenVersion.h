// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Containers/UnrealString.h"

namespace UE::Zen
{

struct FZenVersion
{
	uint32 MajorVersion = 0;
	uint32 MinorVersion = 0;
	uint32 PatchVersion = 0;
	FString Details;

	void Reset();
	bool TryParse(const TCHAR* InString);
	FString ToString(bool bDetailed = true) const;
	bool operator<(FZenVersion& Other) const;
	operator bool() const;
};

} // namespace UE::Zen
