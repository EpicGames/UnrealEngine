// Copyright Epic Games, Inc. All Rights Reserved.

#include "Experimental/BuildListRetriever.h"
#include "Misc/App.h"
#include "Misc/DateTime.h"
#include "Misc/EngineVersion.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Internationalization/Internationalization.h"

#define LOCTEXT_NAMESPACE "FBuildListRetriever"

namespace UE::Zen::Build
{

FBuildListRetriever::FBuildListRetriever()
{
	ServiceInstance = MakeShared<FBuildServiceInstance>();
}

void FBuildListRetriever::ConnectToBuildService()
{
	if (bConnected)
		return;

	ServiceInstance->OnRefreshNamespacesAndBucketsComplete().AddSP(this, &FBuildListRetriever::OnNamespacesAndBucketsRefreshed);
	ServiceInstance->Connect(!FApp::IsUnattended(), [this]
		(UE::Zen::Build::FBuildServiceInstance::EConnectionState ConnectionState,
			UE::Zen::Build::FBuildServiceInstance::EConnectionFailureReason FailureReason)
		{
			if (ConnectionState == UE::Zen::Build::FBuildServiceInstance::EConnectionState::ConnectionSucceeded)
			{
				bConnected = true;
				ServiceInstance->RefreshNamespacesAndBuckets();
			}
		});
}

bool FBuildListRetriever::IsConnected() const
{
	return bConnected;
}

TSharedRef<UE::Zen::Build::FBuildServiceInstance> FBuildListRetriever::GetServiceInstance() const
{
	return ServiceInstance.ToSharedRef();
}

bool FBuildListRetriever::IsReadyForQuery() const
{
	return ReadyToIssueBuildQuery;
}

void FBuildListRetriever::SetOnReadyForQuery( FOnReadyForQuery&& InOnReadyForQuery )
{
	OnReadyForQuery = MoveTemp(InOnReadyForQuery);
}

void FBuildListRetriever::Refresh()
{
	if (bConnected && ReadyToIssueBuildQuery)
	{
		ReadyToIssueBuildQuery = false;
		ServiceInstance->RefreshNamespacesAndBuckets();
	}
}

void FBuildListRetriever::OnNamespacesAndBucketsRefreshed()
{
	ExecuteOnGameThread(UE_SOURCE_LOCATION,
		[this]
		{
			ReadyToIssueBuildQuery = true;

			if (OnReadyForQuery)
			{
				OnReadyForQuery();
			}
		});
}

bool FBuildListRetriever::QueryBuilds(const FString& InProjectName, const FString& InBuildType, const FString& InStream, FOnQueriesComplete&& InOnQueriesCompleteCallback)
{
	check(IsInGameThread());
	if (ReadyToIssueBuildQuery)
	{
		ProjectName = InProjectName;
		BuildType = InBuildType;
		Stream = InStream;
		OnQueriesCompleteCallback = MoveTemp(InOnQueriesCompleteCallback);

		IssueBuildQueries(false);
		return true;
	}
	else
	{
		// Don't do anything if an existing query is in flight
		return false;
	}
}

bool FBuildListRetriever::QueryBuildGroups(const FString& InProjectName, const FString& InBuildType, const FString& InStream, FOnGroupQueriesComplete&& InOnQueriesCompleteCallback)
{
	check(IsInGameThread());
	if (ReadyToIssueBuildQuery)
	{
		ProjectName = InProjectName;
		BuildType = InBuildType;
		Stream = InStream;
		OnGroupQueriesCompleteCallback = MoveTemp(InOnQueriesCompleteCallback);

		IssueBuildQueries(true);
		return true;
	}
	else
	{
		// Don't do anything if an existing query is in flight
		return false;
	}
}

void FBuildListRetriever::IssueBuildQueries(bool bIsGroupQuery)
{
	check(IsInGameThread());

	TMultiMap<FString, FString> NamespacesAndBuckets = ServiceInstance->GetNamespacesAndBuckets();
	
	FString BucketRegex = FString::Printf(TEXT("%s\\.%s\\.%s\\..*"), *ProjectName, *BuildType, *Stream);
	FString BucketPrefix = FString::Printf(TEXT("%s.%s.%s."), *ProjectName, *BuildType, *Stream);
	TArray<FString> Namespaces;
	TArray<FString> NamespacesToQuery;
	NamespacesAndBuckets.GetKeys(Namespaces);

	for (const FString& Namespace : Namespaces)
	{
		TArray<FString> BucketsInNamespace;
		NamespacesAndBuckets.MultiFind(Namespace, BucketsInNamespace);
		for (const FString& Bucket : BucketsInNamespace)
		{
			if (Bucket.StartsWith(BucketPrefix))
			{
				NamespacesToQuery.Add(Namespace);
				break;
			}
		}
	}

	if (NamespacesToQuery.Num() > 0)
	{
		ReadyToIssueBuildQuery = false;
		
		TSharedPtr<FListBuildsState> PendingQueryState = MakeShared<FListBuildsState>();
		PendingQueryState->PendingQueries = NamespacesToQuery.Num();
		PendingQueryState->QueryState.SetNum(NamespacesToQuery.Num());
		uint32 QueryIndex = 0;

		for (const FString& Namespace : NamespacesToQuery)
		{
			FCbWriter QueryWriter;
			QueryWriter.BeginObject();
			if (!bIsGroupQuery)
			{
				QueryWriter.BeginObject("isPreflight");
				QueryWriter.AddString("$eq", "false");
				QueryWriter.EndObject();// isPreflight
			}
			QueryWriter.EndObject();// query

			ServiceInstance->ListBuildsAcrossBuckets(Namespace, BucketRegex, QueryWriter.Save().AsObject(),
				[this, bIsGroupQuery, QueryIndex, Namespace = FString(Namespace),
				PendingQueryState]
				(TArray<FBuildServiceInstance::FBuildRecord>&& Results) mutable
				{
					FBuildState& NewBuildState = PendingQueryState->QueryState[QueryIndex];
					NewBuildState.Namespace = MoveTemp(Namespace);
					NewBuildState.Results = MoveTemp(Results);

					if (--PendingQueryState->PendingQueries == 0)
					{
						// All queries complete
						if (bIsGroupQuery)
						{
							TArray<TSharedRef<FBuildGroup>> BuildGroups = ExtractBuildGroupsFromQueryResult(*PendingQueryState);

							ExecuteOnGameThread(UE_SOURCE_LOCATION,
								[this, BuildGroups = MoveTemp(BuildGroups)]() mutable
								{
									if (OnGroupQueriesCompleteCallback)
									{
										OnGroupQueriesCompleteCallback(BuildGroups);
										OnGroupQueriesCompleteCallback = nullptr;
									}
									ReadyToIssueBuildQuery = true;
								});

						}
						else
						{
							ExtractBuildsFromQueryResult(*PendingQueryState);

							ExecuteOnGameThread(UE_SOURCE_LOCATION,
								[this]
								{
									if (OnQueriesCompleteCallback)
									{
										OnQueriesCompleteCallback(PerPlatformBuilds);
										OnQueriesCompleteCallback = nullptr;
									}
									ReadyToIssueBuildQuery = true;
								});
						}
					}
				});

			++QueryIndex;
		}
	}
	else
	{
		// no namespaces found - send empty results
		if (bIsGroupQuery)
		{
			ExecuteOnGameThread(UE_SOURCE_LOCATION,
				[this]() mutable
				{
					if (OnGroupQueriesCompleteCallback)
					{
						TArray<TSharedRef<FBuildGroup>> BuildGroups;
						OnGroupQueriesCompleteCallback(BuildGroups);
						OnGroupQueriesCompleteCallback = nullptr;
					}
					ReadyToIssueBuildQuery = true;
				});

		}
		else
		{
			PerPlatformBuilds.Reset();

			ExecuteOnGameThread(UE_SOURCE_LOCATION,
				[this]
				{
					if (OnQueriesCompleteCallback)
					{
						OnQueriesCompleteCallback(PerPlatformBuilds);
						OnQueriesCompleteCallback = nullptr;
					}
					ReadyToIssueBuildQuery = true;
				});
		}
	}
}



void FBuildListRetriever::ExtractBuildsFromQueryResult(FListBuildsState& ListBuildsState)
{
	PerPlatformBuilds.Reset();
	for (FBuildState& BuildState : ListBuildsState.QueryState)
	{
		for (FBuildServiceInstance::FBuildRecord& BuildRecord : BuildState.Results)
		{
			TArray<int32>& Builds = PerPlatformBuilds.FindOrAdd(BuildRecord.GetCookPlatform());
			FString CommitIdentifier = BuildRecord.GetCommitIdentifier();
			Builds.Add(FCString::Atoi(*CommitIdentifier));
		}
	}

	// Sort builds in ascending order for each platform
	for (auto& Pair : PerPlatformBuilds)
	{
		Pair.Value.Sort(TLess<int32>());
	}
}

// adapted from SBuildSelection::RegenerateBuildGroups
TArray<TSharedRef<FBuildListRetriever::FBuildGroup>> FBuildListRetriever::ExtractBuildGroupsFromQueryResult(FListBuildsState& ListBuildsState)
{
	TArray<TSharedRef<FBuildGroup>> Result;

	auto MakeGroupKey = [](const FString& Namespace, const FString& CommitIdentifier, const FBuildServiceInstance::FBuildRecord& BuildRecord)
	{
		if (FCbFieldView BuildGroupField = BuildRecord.Metadata.FindIgnoreCase("buildgroup"); BuildGroupField.HasValue() && !BuildGroupField.HasError())
		{
			return FString(WriteToString<64>(Namespace, ".", BuildGroupField.AsString()));
		}
		if (FCbFieldView NameField = BuildRecord.Metadata.FindIgnoreCase("name"); NameField.HasValue() && !NameField.HasError())
		{
			FString CandidateKeyName = FString(WriteToString<64>(Namespace, ".", NameField.AsString()));
			// TODO: This name manipulation needs to be removed when the metadata is more consistent.
			int32 CLIndex = CandidateKeyName.Find(TEXT("-CL-"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
			if (CLIndex != INDEX_NONE)
			{
				int32 DashIndex = CandidateKeyName.Find(TEXT("-"), ESearchCase::IgnoreCase, ESearchDir::FromStart, CLIndex + 4);
				if (DashIndex != INDEX_NONE)
				{
					CandidateKeyName.LeftInline(DashIndex);
				}
				else
				{
					int32 DotIndex = CandidateKeyName.Find(TEXT("."), ESearchCase::IgnoreCase, ESearchDir::FromStart, CLIndex + 4);
					if (DotIndex != INDEX_NONE)
					{
						CandidateKeyName.LeftInline(DotIndex);
					}
				}
			}
			return CandidateKeyName;
		}
		if (!CommitIdentifier.IsEmpty())
		{
			return FString(WriteToString<64>(Namespace, ".", CommitIdentifier));
		}
		return FString(WriteToString<64>(BuildRecord.BuildId));
	};
	TMap<FString, TSharedPtr<FBuildGroup>> KeyedGroups;




	for (FBuildState& BuildState : ListBuildsState.QueryState)
	{
		for (FBuildServiceInstance::FBuildRecord& BuildRecord : BuildState.Results)
		{
			FString CommitIdentifier = BuildRecord.GetCommitIdentifier();
			int32 Changelist = FCString::Atoi(*CommitIdentifier);

			FString GroupKey = MakeGroupKey(BuildState.Namespace, CommitIdentifier, BuildRecord);
			TSharedPtr<FBuildGroup>& BuildGroup = KeyedGroups.FindOrAdd(GroupKey);
			if (!BuildGroup)
			{
				BuildGroup = MakeShared<FBuildGroup>();
			}

			FString ArtifactName;
			if (FCbFieldView NameView = BuildRecord.Metadata.FindIgnoreCase("name"); NameView.HasValue() && !NameView.HasError())
			{
				ArtifactName = FUTF8ToTCHAR(NameView.AsString());

				// TODO: This name manipulation needs to be removed when the metadata is more consistent.
				int32 TruncationIndex;
				int32 CLIndex = ArtifactName.Find(TEXT("-CL-"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
				if (CLIndex != INDEX_NONE)
				{
					ArtifactName.RightChopInline(CLIndex + 4);
					if (ArtifactName.FindChar(TCHAR('-'), TruncationIndex))
					{
						ArtifactName.RightChopInline(TruncationIndex + 1);
					}
				}
				if (ArtifactName.FindLastChar(TCHAR('.'), TruncationIndex))
				{
					ArtifactName.RightChopInline(TruncationIndex + 1);
				}
			}

			if (BuildGroup->DisplayName.IsEmpty())
			{
				FString Job;
				if (FCbFieldView JobField = BuildRecord.Metadata.FindIgnoreCase("job"); JobField.HasValue() && !JobField.HasError())
				{
					Job = FUTF8ToTCHAR(JobField.AsString());
				}

				FString Category;
				if (FCbFieldView TemplateIdField = BuildRecord.Metadata.FindIgnoreCase("hordeTemplateId"); TemplateIdField.HasValue() && !TemplateIdField.HasError())
				{
					Category = FUTF8ToTCHAR(TemplateIdField.AsString());
				}

				bool bIsPreflight = false;
				if (FCbFieldView IsPreflightField = BuildRecord.Metadata.FindIgnoreCase("ispreflight"); IsPreflightField.HasValue() && !IsPreflightField.HasError())
				{
					FString IsPreflightString;
					IsPreflightString = FUTF8ToTCHAR(IsPreflightField.AsString());
					bIsPreflight = IsPreflightString.ToBool();
				}

				FDateTime CreatedAt;
				if (FCbFieldView CreatedAtField = BuildRecord.Metadata.FindIgnoreCase("createdAt"); CreatedAtField.HasValue() && !CreatedAtField.HasError())
				{
					if (CreatedAtField.IsString())
					{
						FDateTime::ParseIso8601(FUTF8ToTCHAR(CreatedAtField.AsString()).Get(), CreatedAt);
					}
					else if (CreatedAtField.IsDateTime())
					{
						CreatedAt = CreatedAtField.AsDateTime();
					}
				}

				FString Configuration;
				if (FCbFieldView ConfigurationField = BuildRecord.Metadata.FindIgnoreCase("Configuration"); ConfigurationField.HasValue() && !ConfigurationField.HasError())
				{
					Configuration = FUTF8ToTCHAR(ConfigurationField.AsString());
				}

				FString BuildVersionSuffix;
				if (FCbFieldView BuildVersionSuffixField = BuildRecord.Metadata.FindIgnoreCase("BuildVersionSuffix"); BuildVersionSuffixField.HasValue() && !BuildVersionSuffixField.HasError())
				{
					BuildVersionSuffix = FUTF8ToTCHAR(BuildVersionSuffixField.AsString());
					BuildVersionSuffix.RemoveFromStart(TEXT("-"));
				}


				FString ItemName;
				FCbFieldView GroupNameView = BuildRecord.Metadata.FindIgnoreCase("buildgroup");
				if (!GroupNameView.HasValue())
				{
					GroupNameView = BuildRecord.Metadata.FindIgnoreCase("name");
				}
				if (FCbFieldView NameView = GroupNameView; NameView.HasValue() && !NameView.HasError())
				{
					// TODO: This name manipulation needs to be removed when the metadata is more consistent.
					ItemName = FUTF8ToTCHAR(NameView.AsString());
				}
				else
				{
					ItemName = *WriteToString<64>(BuildRecord.BuildId);
				}

				BuildGroup->Namespace = BuildState.Namespace;
				BuildGroup->DisplayName = ItemName;
				BuildGroup->CommitIdentifier = CommitIdentifier;
				BuildGroup->Changelist = FCString::Atoi(*CommitIdentifier);
				BuildGroup->Category = Category;
				BuildGroup->CreatedAt = CreatedAt;
				BuildGroup->Job = Job;
				BuildGroup->Suffix = BuildVersionSuffix;
				BuildGroup->bIsPreflight = bIsPreflight;
			}

			BuildGroup->NamedArtifacts.FindOrAdd(ArtifactName, MoveTemp(BuildRecord));
		}
	}

	TArray<TSharedPtr<FBuildGroup>> BuildGroups;
	KeyedGroups.GenerateValueArray(BuildGroups);

	Result.Reserve(BuildGroups.Num());
	for (const TSharedPtr<FBuildGroup>& BuildGroup : BuildGroups)
	{
		Result.Add(BuildGroup.ToSharedRef());
	}

	Result.Sort( [](const TSharedRef<FBuildListRetriever::FBuildGroup>& A, const TSharedRef<FBuildListRetriever::FBuildGroup>& B)
	{
		return A->CreatedAt > B->CreatedAt;
	});

	return MoveTemp(Result);
}



bool FBuildListRetriever::GetBucketInfo( const FString& BucketId, FBucketInfo& OutBucketInfo )
{
	TArray<FString> BucketItems;
	if (BucketId.ParseIntoArray(BucketItems, TEXT(".")) != 4)
	{
		return false;
	}

	OutBucketInfo.Project = BucketItems[0];
	OutBucketInfo.BuildType = BucketItems[1];
	OutBucketInfo.Stream = BucketItems[2];
	OutBucketInfo.Platform = BucketItems[3];
	return true;
}



TMap<FString,TSharedRef<FBuildListRetriever::FBuildType, ESPMode::ThreadSafe>> FBuildListRetriever::GenerateBuildTypesMap() const
{
	// gather all build type names
	TSet<FString> BuildTypeNames;
	for (const auto& Pair : ServiceInstance->GetNamespacesAndBuckets())
	{
		const FString& Namespace = Pair.Key;
		const FString& BucketId = Pair.Value;

		FBucketInfo BucketInfo;
		if (GetBucketInfo(BucketId, BucketInfo))
		{
			if (BucketInfo.Stream == Stream && BucketInfo.Project == ProjectName)
			{
				BuildTypeNames.Add(BucketInfo.BuildType);
			}
		}
	}

	// build the sorted list of build types
	TMap<FString,TSharedRef<FBuildType>> Result;
	for (const FString& BuildTypeName : BuildTypeNames)
	{
		Result.Add( BuildTypeName, MakeBuildType(BuildTypeName) );
	}
	Result.ValueSort( [](const TSharedRef<FBuildType>& A, const TSharedRef<FBuildType>& B)
	{
		if (A->Order != B->Order)
		{
			return A->Order >= B->Order;
		}

		return A->DisplayName.ToString() < B->DisplayName.ToString();
	});


	return Result;
}



TSharedRef<FBuildListRetriever::FBuildType, ESPMode::ThreadSafe> FBuildListRetriever::MakeBuildType(const FString& InBuildTypeName) const
{
	struct FKnownBuildType
	{
		FRegexPattern Pattern;
		EBuildType Type;
		FText DisplayName;
		int Order;
	};
	static const FKnownBuildType KnownBuildTypes[] =
	{
		{ FRegexPattern(TEXT(".*oplog-?(.*)")),				EBuildType::Oplog,						LOCTEXT("BuildSelection_BuildType_Oplog", "Cook Snapshot"), 0 },
		{ FRegexPattern(TEXT(".*packaged-?build-?(.*)")),	EBuildType::PackagedBuild,				LOCTEXT("BuildSelection_BuildType_PackagedBuild", "Packaged Build"), 1},
		{ FRegexPattern(TEXT(".*staged-?(.*?)-?build.*")),	EBuildType::StagedBuild,				LOCTEXT("BuildSelection_BuildType_StagedBuild", "Staged Build"), 2},
		{ FRegexPattern(TEXT(".*ugs-?pcb-?(.*)")),			EBuildType::EditorPreCompiledBinary,	LOCTEXT("BuildSelection_BuildType_PreCompiledBinary", "Editor Pre-compiled Binary"), 0},
		{ FRegexPattern(TEXT(".*installed-?build-?(.*)")),	EBuildType::EditorPreCompiledBinary,	LOCTEXT("BuildSelection_BuildType_PreCompiledBinary", "Editor Pre-compiled Binary"), 0},
	};


	TSharedRef<FBuildType, ESPMode::ThreadSafe> Result = MakeShared<FBuildType, ESPMode::ThreadSafe>();

	for (int32 Index = 0; Index < UE_ARRAY_COUNT(KnownBuildTypes); Index++)
	{
		const FKnownBuildType& KnownBuildType = KnownBuildTypes[Index];
		FRegexMatcher Matcher(KnownBuildType.Pattern, InBuildTypeName);
		if (Matcher.FindNext())
		{
			Result->Type = KnownBuildType.Type;
			Result->Order = KnownBuildType.Order;
			Result->DisplayName = KnownBuildType.DisplayName;

			FString OptionalCaptureGroupString = Matcher.GetCaptureGroup(1);
			if (!OptionalCaptureGroupString.IsEmpty())
			{
				Result->DisplayName = FText::Format(LOCTEXT("BuildSelection_KnownBuildTypeWithCaptureGroupFormat", "{0} ({1})"), KnownBuildType.DisplayName, FText::FromString(OptionalCaptureGroupString));
			}

			return Result;
		}
	}

	// unknown build tye
	Result->Type = EBuildType::Unknown;
	Result->DisplayName = FText::FromString(*InBuildTypeName);
	Result->Order = 0;
	
	return Result;
}



} // namespace UE::Zen::Build

#undef LOCTEXT_NAMESPACE
