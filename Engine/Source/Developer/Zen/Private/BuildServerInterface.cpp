// Copyright Epic Games, Inc. All Rights Reserved.

#include "Experimental/BuildServerInterface.h"

#include "Async/Mutex.h"
#include "Containers/StringView.h"
#include "DesktopPlatformModule.h"
#include "Experimental/ZenProjectStoreWriter.h"
#include "Experimental/ZenServerInterface.h"
#include "HAL/CriticalSection.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformOutputDevices.h"
#include "Http/HttpClient.h"
#include "Http/HttpHostBuilder.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "Misc/MonitoredProcess.h"
#include "Misc/OutputDeviceFile.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/ScopeExit.h"
#include "Misc/ScopeLock.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/StringBuilder.h"
#include "Serialization/Archive.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryValidation.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "SocketSubsystem.h"
#include "String/LexFromString.h"
#include "Tasks/Task.h"
#include "StudioTelemetry.h"

#if PLATFORM_WINDOWS
#	include "Windows/AllowWindowsPlatformTypes.h"
#	include <Windows.h>
#	include "Windows/HideWindowsPlatformTypes.h"
#elif PLATFORM_MAC || PLATFORM_LINUX
#	include <signal.h>
#	include <errno.h>
#endif

#if WITH_SSL
#include "Ssl.h"
#endif

#define UE_BUILDSERVERINSTANCE_MAX_FAILED_LOGIN_ATTEMPTS 16

namespace UE::Zen::Build
{

DEFINE_LOG_CATEGORY_STATIC(LogBuildServiceInstance, Log, All);

constexpr double GracefulTerminationTimeoutSeconds = 15.0;

bool SendSoftTerminationSignal(FProcHandle ProcessHandle)
{
#if PLATFORM_WINDOWS
	// FPlatformProcess::CreateProc always sets CREATE_NEW_PROCESS_GROUP, which
	// causes Windows to implicitly disable CTRL_C_EVENT for the child. We use
	// CTRL_BREAK_EVENT instead, which cannot be disabled and can target a
	// specific process group (the child's PID equals its process group ID).
	//
	// GenerateConsoleCtrlEvent requires the caller to share a console with the
	// target. The child is launched with bLaunchDetached=false, so it inherits
	// our console when we have one. If we don't (e.g. GUI host), AttachConsole
	// lets us temporarily join the child's console. We avoid calling FreeConsole
	// up front to prevent destroying our own console — that would invalidate
	// stdout/stderr handles and break concurrent I/O on other threads.
	DWORD ChildPid = ::GetProcessId(ProcessHandle.Get());
	if (ChildPid == 0)
	{
		return false;
	}

	// Try to attach to the child's console. If we already share one,
	// AttachConsole fails with ERROR_ACCESS_DENIED and we can proceed directly.
	bool bDidAttach = !!::AttachConsole(ChildPid);
	if (!bDidAttach && ::GetLastError() != ERROR_ACCESS_DENIED)
	{
		// Child has no console — cannot send a console control event.
		return false;
	}

	// CTRL_BREAK_EVENT with a non-zero process group ID only targets that
	// group, so our own process will not receive the signal.
	BOOL bResult = ::GenerateConsoleCtrlEvent(CTRL_BREAK_EVENT, ChildPid);

	if (bDidAttach)
	{
		::FreeConsole();
	}

	return bResult != 0;

#elif PLATFORM_MAC || PLATFORM_LINUX
	pid_t Pid = ProcessHandle.Get();
	if (Pid <= 0)
	{
		return false;
	}
	return kill(Pid, SIGINT) == 0;

#else
	return false;
#endif
}

namespace Private
{

struct FProgressState
{
	FString Label;
	FString Detail;
	float Percent = 0.0f;
	float Bandwidth = 0.0f;
};

struct FBuildTransferState : public TSharedFromThis<FBuildTransferState, ESPMode::ThreadSafe>
{
	FBuildServiceInstance::FBuildTransfer::EType Type;
	FCbObjectId BuildId;
	FString Name;
	FString Namespace;
	FString Bucket;
	FString Destination;
	FString RootDir;
	FString EngineDir;
	FString ProjectFilePath;
	FString IncludeFilter;
	FString ExcludeFilter;
	FName TargetPlatformName = NAME_None;
	FString DownloadSpecJSONContents;
	TArray<FString> PartIds;
	TArray<FString> PartNames;
	FString HostOverride;
	EBuildTransferRequestFlags RequestFlags;
	double StartTime=0;

	std::atomic<bool> bCancelRequested = false;

	mutable FRWLock Lock;
	TArray<FProgressState> ProgressState;
	const uint32 RecentOutputLinesCapacity = 16;
	TCircularBuffer<FString> RecentOutputLines = TCircularBuffer<FString>(RecentOutputLinesCapacity);
	uint32 NextOutputLineIndex = 0;
	TUniquePtr<FArchive> OutputLogFile;
	UE::Zen::Build::FBuildServiceInstance::EBuildTransferStatus Status = UE::Zen::Build::FBuildServiceInstance::EBuildTransferStatus::Queued;
	int ReturnCode=-1;
	FMonotonicTimePoint TransferStartTime;
	FMonotonicTimePoint TransferEndTime;

	FString GetRecentOutput() const;
	FString GetLogFilename() const;
};

class FAccessToken
{
public:
	void SetToken(FStringView Scheme, FStringView Token);
	inline uint32 GetSerial() const { return Serial.load(std::memory_order_relaxed); }
	friend FAnsiStringBuilderBase& operator<<(FAnsiStringBuilderBase& Builder, const FAccessToken& Token);

private:
	mutable FRWLock Lock;
	TArray<ANSICHAR> Header;
	std::atomic<uint32> Serial;
};

FAnsiStringBuilderBase& operator<<(FAnsiStringBuilderBase& Builder, const Private::FAccessToken& Token)
{
	FReadScopeLock ReadLock(Token.Lock);
	return Builder.Append(Token.Header);
}

FString
FBuildTransferState::GetRecentOutput() const
{
	TStringBuilder<256> StringBuilder;
	if (NextOutputLineIndex == 0)
	{
		return FString();
	}

	uint32 StartIndex = 0;
	if (NextOutputLineIndex > RecentOutputLinesCapacity)
	{
		StartIndex = NextOutputLineIndex - RecentOutputLinesCapacity + 1;
	}

	for (uint32 Index = StartIndex; Index < NextOutputLineIndex; ++Index)
	{
		StringBuilder.Append(RecentOutputLines[Index]);
		if (Index < (NextOutputLineIndex - 1))
		{
			StringBuilder.Append(LINE_TERMINATOR);
		}
	}

	return StringBuilder.ToString();
}

FString
FBuildTransferState::GetLogFilename() const
{
	const FString LogDirectory = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(
		*FPaths::GetPath(FPlatformOutputDevices::GetAbsoluteLogFilename())
	);
	return FPaths::Combine(LogDirectory, TEXT("Transfers"), FPaths::MakeValidFileName(Name)) + TEXT(".log");
}

void FAccessToken::SetToken(const FStringView Scheme, const FStringView Token)
{
	FWriteScopeLock WriteLock(Lock);
	const int32 TokenLen = FPlatformString::ConvertedLength<ANSICHAR>(Token.GetData(), Token.Len());

	Header.Empty(TokenLen);

	const int32 TokenIndex = Header.AddUninitialized(TokenLen);
	FPlatformString::Convert(Header.GetData() + TokenIndex, TokenLen, Token.GetData(), Token.Len());
	Serial.fetch_add(1, std::memory_order_relaxed);
}
} // namespace Private

bool LoadFromCompactBinary(FCbFieldView Field, FStringView BucketId, FBuildServiceInstance::FBuildRecord& OutBuildRecord)
{
	if (!Field.IsObject())
	{
		return false;
	}
	bool bOk = true;
	FString BuildIdString;
	FCbObjectId::ByteArray BuildIdBytes;
	bOk &= LoadFromCompactBinary(Field["buildId"], BuildIdString);
	if (BucketId.IsEmpty())
	{
		bOk &= LoadFromCompactBinary(Field["bucketId"], OutBuildRecord.BucketId);
	}
	else
	{
		OutBuildRecord.BucketId = BucketId;
	}
	bOk &= UE::String::HexToBytes(BuildIdString, BuildIdBytes) == sizeof(BuildIdBytes);
	OutBuildRecord.BuildId = FCbObjectId(BuildIdBytes);

	FCbFieldView MetadataField = Field["metadata"];
	if (!MetadataField.IsObject())
	{
		return false;
	}

	OutBuildRecord.Metadata = FCbObject::Clone(MetadataField.AsObjectView());
	return bOk;
}

bool
FServiceSettings::ReadFromConfig()
{
	check(GConfig && GConfig->IsReadyForUse());
	FString Config;
	if (!GConfig->GetString(TEXT("StorageServers"), TEXT("Cloud"), Config, GEngineIni))
	{
		return false;
	}
	bool bRetVal = false;

	bRetVal |= FParse::Value(*Config, TEXT("Host="), Host);
	bRetVal |= FParse::Value(*Config, TEXT("OAuthProviderIdentifier="), OAuthProviderIdentifier);

	FParse::Value(*Config, TEXT("AuthScheme="), AuthScheme);
	if (AuthScheme.IsEmpty())
	{
		AuthScheme = "Bearer";
	}

	return bRetVal;
}

bool
FServiceSettings::ReadFromURL(FStringView InstanceURL)
{
	Host = InstanceURL;
	return true;
}

FBuildServiceInstance::FBuildServiceInstance()
{
	Settings.ReadFromConfig();
	Initialize();
}

FBuildServiceInstance::FBuildServiceInstance(FStringView InstanceURL)
{
	Settings.ReadFromURL(InstanceURL);
	Initialize();
}

FBuildServiceInstance::~FBuildServiceInstance()
{
	bBuildTransferThreadStopping = true;
	if (BuildTransferThread.IsJoinable())
	{
		BuildTransferThread.Join();
	}
}

void
FBuildServiceInstance::Connect(bool bAllowInteractive, FOnConnectionComplete&& OnConnectionComplete)
{
	const FString Host = Settings.GetHost();
	if (Host.IsEmpty() || (Host == TEXT("None")))
	{
		if (OnConnectionComplete)
		{
			OnConnectionComplete(EConnectionState::ConnectionFailed, EConnectionFailureReason::MissingHost);
		}
		return;
	}

	if (ConnectionState.exchange(EConnectionState::ConnectionInProgress, std::memory_order_relaxed) == EConnectionState::ConnectionInProgress)
	{
		if (OnConnectionComplete)
		{
			OnConnectionComplete(EConnectionState::ConnectionFailed, EConnectionFailureReason::UnexpectedState);
		}
		return;
	}

	UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, bAllowInteractive, OnConnectionComplete = MoveTemp(OnConnectionComplete)]()
	{
		TAnsiStringBuilder<256> ResolvedHost;
		double ResolvedLatency;
		FHttpHostBuilder HostBuilder;
		HostBuilder.AddFromString(Settings.GetHost());
		bool bHostResolvingSucceeded = HostBuilder.ResolveHost(/* Warning timeout */ 1.0, 4.0 /* Max duration timeout*/, ResolvedHost, ResolvedLatency);
		
		// even if we fail to resolve a host to use the returned host will at least contain the first of the possible hosts which we can attempt to use
		EffectiveDomain = ResolvedHost;

		if (!bHostResolvingSucceeded)
		{
			FString HostCandidates = HostBuilder.GetHostCandidatesString();
			const FString MessageString = FString::Printf(TEXT("Unable to resolve best host candidate to use. Most likely none of the suggested hosts was reachable. Attempted hosts were: '%s' ."), *HostBuilder.GetHostCandidatesString() );
			UE_LOGF(LogBuildServiceInstance, Warning, "%ls", *MessageString);
		}

		EDesktopLoginInteractionLevel InteractionLevel = bAllowInteractive ? EDesktopLoginInteractionLevel::Interactive : EDesktopLoginInteractionLevel::None;
		if (AcquireAccessToken(InteractionLevel))
		{
			RecordConnectionEvent(true);
			ConnectionState.store(EConnectionState::ConnectionSucceeded, std::memory_order_relaxed);
			if (OnConnectionComplete)
			{
				OnConnectionComplete(EConnectionState::ConnectionSucceeded, EConnectionFailureReason::None);
			}
		}
		else
		{
			const FString MessageString = TEXT("Unable to acquire access token.");
			UE_LOGF(LogBuildServiceInstance, Warning, "%ls", *MessageString);
			RecordConnectionEvent(false, MessageString);
			ConnectionState.store(EConnectionState::ConnectionFailed, std::memory_order_relaxed);
			if (OnConnectionComplete)
			{
				OnConnectionComplete(EConnectionState::ConnectionFailed, EConnectionFailureReason::FailedToAcquireAccessToken);
			}
		}

	});
}

void
FBuildServiceInstance::RefreshNamespacesAndBuckets(FOnRefreshNamespacesAndBucketsComplete&& InOnRefreshNamespacesAndBucketsComplete)
{
	{
		FWriteScopeLock _(NamespacesAndBucketsLock);
		NamespacesAndBuckets.Empty();
	}
	if (ConnectionState.load(std::memory_order_relaxed) != EConnectionState::ConnectionSucceeded)
	{
		CallOnRefreshNamespacesAndBucketsComplete(MoveTemp(InOnRefreshNamespacesAndBucketsComplete));
		return;
	}

	FString OutputFilePath = FPaths::CreateTempFilename(FPlatformProcess::UserTempDir(), TEXT("BuildContainers"), TEXT(".cbo"));
	FPaths::MakePlatformFilename(OutputFilePath);

	FString CommandLineArgs = FString::Printf(TEXT("builds list-namespaces --recursive \"%s\""),
		*OutputFilePath);
	CommandLineArgs = AddZenUtilityBuildServerArguments(*CommandLineArgs);

	InvokeZenUtility(CommandLineArgs, nullptr,
	[this, OutputFilePath = MoveTemp(OutputFilePath), InOnRefreshNamespacesAndBucketsComplete = MoveTemp(InOnRefreshNamespacesAndBucketsComplete)](bool bSuccessful) mutable
	{
		if (bSuccessful)
		{
			TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileReader(*OutputFilePath));
			if (!Ar)
			{
				UE_LOGF(LogBuildServiceInstance, Warning, "Missing output file from zen utility when gathering build data: '%ls'", *OutputFilePath);
				CallOnRefreshNamespacesAndBucketsComplete(MoveTemp(InOnRefreshNamespacesAndBucketsComplete));
				return;
			}

			FCbObject OutputObject = LoadCompactBinary(*Ar).AsObject();
			{
				FWriteScopeLock _(NamespacesAndBucketsLock);
				for (FCbFieldView ResultField : OutputObject["results"])
				{
					FString Namespace = *WriteToString<64>(ResultField["name"].AsString());
					for (FCbFieldView Item : ResultField["items"])
					{
						NamespacesAndBuckets.Add(Namespace, *WriteToString<64>(Item.AsString()));
					}
				}
			}
		}
		CallOnRefreshNamespacesAndBucketsComplete(MoveTemp(InOnRefreshNamespacesAndBucketsComplete));
	});
}

void
FBuildServiceInstance::ListBuilds(FStringView Namespace, FStringView Bucket, FCbObject&& Query, FOnListBuildsComplete&& InOnListBuildsComplete)
{
	if (ConnectionState.load(std::memory_order_relaxed) != EConnectionState::ConnectionSucceeded)
	{
		if (InOnListBuildsComplete)
		{
			InOnListBuildsComplete({});
		}
		return;
	}

	FString QueryFilePath = FPaths::CreateTempFilename(FPlatformProcess::UserTempDir(), TEXT("BuildListQuery"), TEXT(".cbo"));
	FPaths::MakePlatformFilename(QueryFilePath);

	FString OutputFilePath = FPaths::CreateTempFilename(FPlatformProcess::UserTempDir(), TEXT("BuildList"), TEXT(".cbo"));
	FPaths::MakePlatformFilename(OutputFilePath);

	FString BucketParam;
	if (!Bucket.IsEmpty())
	{
		BucketParam = FString::Printf(TEXT(" --bucket \"%s\""), *WriteToString<64>(Bucket));
	}

	FString CommandLineArgs = FString::Printf(TEXT("builds list --namespace \"%s\"%s \"%s\" \"%s\""),
		*WriteToString<64>(Namespace), *BucketParam, *QueryFilePath, *OutputFilePath);
	CommandLineArgs = AddZenUtilityBuildServerArguments(*CommandLineArgs);

	auto PreInvoke = [QueryFilePath, Query = MoveTemp(Query)](FString& CommandLineArgs)
	{
		TUniquePtr<FArchive> QueryFileArchive(IFileManager::Get().CreateFileWriter(*QueryFilePath, FILEWRITE_NoFail));
		FCbWriter Writer;
		Writer.BeginObject();
		Writer.AddObject("query", Query);
		Writer.EndObject();
		Writer.Save(*QueryFileArchive);
	};

	InvokeZenUtility(CommandLineArgs, MoveTemp(PreInvoke),
	[QueryFilePath = MoveTemp(QueryFilePath), OutputFilePath = MoveTemp(OutputFilePath), Bucket = FString(Bucket), InOnListBuildsComplete = MoveTemp(InOnListBuildsComplete)](bool bSuccessful) mutable
	{
		ON_SCOPE_EXIT
		{
			IFileManager & FileManager = IFileManager::Get();
			FileManager.Delete(*QueryFilePath, false, false, true);
			FileManager.Delete(*OutputFilePath, false, false, true);
		};

		if (!bSuccessful)
		{
			if (InOnListBuildsComplete)
			{
				InOnListBuildsComplete({});
			}
			return;
		}

		TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileReader(*OutputFilePath));
		if (!Ar)
		{
			UE_LOGF(LogBuildServiceInstance, Warning, "Missing output file from zen utility when gathering build data: '%ls'", *OutputFilePath);
			if (InOnListBuildsComplete)
			{
				InOnListBuildsComplete({});
			}
			return;
		}

		if (InOnListBuildsComplete)
		{
			FCbObject OutputObject = LoadCompactBinary(*Ar).AsObject();
			TArray<FBuildRecord> BuildRecords;
			for (FCbFieldView ResultField : OutputObject["results"].AsArrayView())
			{
				FBuildRecord BuildRecord;
				if (LoadFromCompactBinary(ResultField, Bucket, BuildRecord))
				{
					BuildRecords.Emplace(MoveTemp(BuildRecord));
				}
			}
			InOnListBuildsComplete(MoveTemp(BuildRecords));
		}
	});
}

void
FBuildServiceInstance::ListBuildsAcrossBuckets(FStringView Namespace, FStringView BucketRegex, FCbObject&& Query, FOnListBuildsComplete&& InOnListBuildsComplete)
{
	if (ConnectionState.load(std::memory_order_relaxed) != EConnectionState::ConnectionSucceeded)
	{
		if (InOnListBuildsComplete)
		{
			InOnListBuildsComplete({});
		}
		return;
	}

	FString QueryFilePath = FPaths::CreateTempFilename(FPlatformProcess::UserTempDir(), TEXT("BuildListQuery"), TEXT(".cbo"));
	FPaths::MakePlatformFilename(QueryFilePath);

	FString OutputFilePath = FPaths::CreateTempFilename(FPlatformProcess::UserTempDir(), TEXT("BuildList"), TEXT(".cbo"));
	FPaths::MakePlatformFilename(OutputFilePath);

	FString CommandLineArgs = FString::Printf(TEXT("builds list --namespace \"%s\" \"%s\" \"%s\""),
		*WriteToString<64>(Namespace), *QueryFilePath, *OutputFilePath);
	CommandLineArgs = AddZenUtilityBuildServerArguments(*CommandLineArgs);

	auto PreInvoke = [QueryFilePath, BucketRegex = FString(BucketRegex), Query = MoveTemp(Query)](FString& CommandLineArgs)
	{
		TUniquePtr<FArchive> QueryFileArchive(IFileManager::Get().CreateFileWriter(*QueryFilePath, FILEWRITE_NoFail));
		FCbWriter Writer;
		Writer.BeginObject();
		Writer.AddString("bucketRegex", BucketRegex);
		Writer.AddObject("query", Query);
		Writer.BeginObject("options");
		Writer.AddInteger("limit", 50000);
		Writer.AddInteger("max", 500000);
		Writer.EndObject();
		Writer.EndObject();
		Writer.Save(*QueryFileArchive);
	};

	InvokeZenUtility(CommandLineArgs, MoveTemp(PreInvoke),
	[QueryFilePath = MoveTemp(QueryFilePath), OutputFilePath = MoveTemp(OutputFilePath), InOnListBuildsComplete = MoveTemp(InOnListBuildsComplete)](bool bSuccessful) mutable
	{
		ON_SCOPE_EXIT
		{
			IFileManager & FileManager = IFileManager::Get();
			FileManager.Delete(*QueryFilePath, false, false, true);
			FileManager.Delete(*OutputFilePath, false, false, true);
		};

		if (!bSuccessful)
		{
			if (InOnListBuildsComplete)
			{
				InOnListBuildsComplete({});
			}
			return;
		}

		TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileReader(*OutputFilePath));
		if (!Ar)
		{
			UE_LOGF(LogBuildServiceInstance, Warning, "Missing output file from zen utility when gathering build data: '%ls'", *OutputFilePath);
			if (InOnListBuildsComplete)
			{
				InOnListBuildsComplete({});
			}
			return;
		}

		if (InOnListBuildsComplete)
		{
			FCbObject OutputObject = LoadCompactBinary(*Ar).AsObject();
			TArray<FBuildRecord> BuildRecords;
			for (FCbFieldView ResultField : OutputObject["results"].AsArrayView())
			{
				FBuildRecord BuildRecord;
				if (LoadFromCompactBinary(ResultField, {}, BuildRecord))
				{
					BuildRecords.Emplace(MoveTemp(BuildRecord));
				}
			}
			InOnListBuildsComplete(MoveTemp(BuildRecords));
		}
	});
}

void
FBuildServiceInstance::ListBuildContents(FStringView Namespace, FStringView Bucket, FStringView BuildId, FOnListBuildContentsComplete&& InOnListBuildContentsComplete)
{
	if (ConnectionState.load(std::memory_order_relaxed) != EConnectionState::ConnectionSucceeded)
	{
		if (InOnListBuildContentsComplete)
		{
			InOnListBuildContentsComplete({});
		}
		return;
	}

	FString OutputFilePath = FPaths::CreateTempFilename(FPlatformProcess::UserTempDir(), TEXT("BuildContents"), TEXT(".cbo"));
	FPaths::MakePlatformFilename(OutputFilePath);

	FString OidcTokenExecutableFilename = FPaths::ConvertRelativePathToFull(FDesktopPlatformModule::TryGet()->GetOidcTokenExecutableFilename(FPaths::RootDir()));

	FString CommandLineArgs = FString::Printf(TEXT("builds ls --namespace %s --bucket %s --build-id %s --result-path \"%s\""),
		*WriteToString<64>(Namespace),
		*WriteToString<64>(Bucket),
		*WriteToString<24>(BuildId),
		*OutputFilePath);
	CommandLineArgs = AddZenUtilityBuildServerArguments(*CommandLineArgs);

	InvokeZenUtility(CommandLineArgs, nullptr,
	[OutputFilePath = MoveTemp(OutputFilePath), InOnListBuildContentsComplete = MoveTemp(InOnListBuildContentsComplete)](bool bSuccessful) mutable
	{
		ON_SCOPE_EXIT
		{
			IFileManager::Get().Delete(*OutputFilePath, false, false, true);
		};

		if (!bSuccessful)
		{
			if (InOnListBuildContentsComplete)
			{
				InOnListBuildContentsComplete({});
			}
			return;
		}

		TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileReader(*OutputFilePath));
		if (!Ar)
		{
			UE_LOGF(LogBuildServiceInstance, Warning, "Missing output file from zen utility when listing build contents: '%ls'", *OutputFilePath);
			if (InOnListBuildContentsComplete)
			{
				InOnListBuildContentsComplete({});
			}
			return;
		}

		if (InOnListBuildContentsComplete)
		{
			FCbObject OutputObject = LoadCompactBinary(*Ar).AsObject();
			TMap<FString, FBuildPart> BuildContents;
			for (FCbFieldView PartField : OutputObject["parts"])
			{
				FString PartName = *WriteToString<64>(PartField["partName"].AsString());

				FBuildPart Part;
				Part.Id = PartField["id"].AsObjectId();

				for (FCbFieldView FileField : PartField["files"])
				{
					FBuildFile File;
					File.Name = *WriteToString<64>(FileField["path"].AsString());
					File.RawSize = FileField["rawSize"].AsUInt64();
					File.Attributes = static_cast<uint32>(FileField["attributes"].AsUInt64());
					Part.Files.Emplace(MoveTemp(File));
				}

				BuildContents.Add(MoveTemp(PartName), MoveTemp(Part));
			}
			InOnListBuildContentsComplete(MoveTemp(BuildContents));
		}
	});
}

FBuildServiceInstance::FBuildTransfer
FBuildServiceInstance::StartBuildTransfer(
	const FCbObjectId& BuildId,
	FStringView Name,
	FStringView DestinationFolder,
	FStringView Namespace,
	FStringView Bucket,
	FStringView IncludeFilter,
	FStringView ExcludeFilter,
	FString&& DownloadSpecJSONContents,
	TArray<FString>&& PartIds,
	TArray<FString>&& PartNames,
	FStringView HostOverride,
	EBuildTransferRequestFlags RequestFlags)
{
	TSharedPtr<Private::FBuildTransferState> NewState = MakeShared<Private::FBuildTransferState>();
	NewState->Type = FBuildTransfer::EType::Files;
	NewState->BuildId = BuildId;
	NewState->Name = Name;
	NewState->Namespace = Namespace;
	NewState->Bucket = Bucket;
	NewState->Destination = DestinationFolder;
	NewState->IncludeFilter = IncludeFilter;
	NewState->ExcludeFilter = ExcludeFilter;
	NewState->DownloadSpecJSONContents = DownloadSpecJSONContents;
	NewState->PartIds = MoveTemp(PartIds);
	NewState->PartNames = MoveTemp(PartNames);
	NewState->HostOverride = HostOverride;
	NewState->RequestFlags = RequestFlags;
	NewState->StartTime = FPlatformTime::Seconds();
	BuildTransferThreadStates.Enqueue(NewState.ToSharedRef());
	KickBuildTransferThread();

	FBuildTransfer NewTransfer;
	NewTransfer.State = NewState;
	return NewTransfer;
}

FBuildServiceInstance::FBuildTransfer
FBuildServiceInstance::StartOplogBuildTransfer(
	const FCbObjectId& BuildId,
	FStringView Name,
	FStringView DestinationProjectId,
	FStringView DestinationOplogId,
	FStringView RootDir,
	FStringView EngineDir,
	FStringView ProjectFilePath,
	FStringView Namespace,
	FStringView Bucket,
	FName TargetPlatformName,
	FStringView HostOverride,
	EBuildTransferRequestFlags RequestFlags)
{
	TSharedPtr<Private::FBuildTransferState> NewState = MakeShared<Private::FBuildTransferState>();
	NewState->Type = FBuildTransfer::EType::Oplog;
	NewState->BuildId = BuildId;
	NewState->Name = Name;
	NewState->Namespace = Namespace;
	NewState->Bucket = Bucket;
	NewState->Destination = *WriteToString<64>(DestinationProjectId, TEXT("/"), DestinationOplogId);
	NewState->StartTime = FPlatformTime::Seconds();
	NewState->RootDir = RootDir;
	NewState->EngineDir = EngineDir;
	NewState->ProjectFilePath = ProjectFilePath;
	NewState->TargetPlatformName = TargetPlatformName;
	NewState->HostOverride = HostOverride;
	NewState->RequestFlags = RequestFlags;
	BuildTransferThreadStates.Enqueue(NewState.ToSharedRef());
	KickBuildTransferThread();

	FBuildTransfer NewTransfer;
	NewTransfer.State = NewState;
	return NewTransfer;
}

FBuildServiceInstance::FBuildTransfer
FBuildServiceInstance::RepeatBuildTransfer(FBuildTransfer BuildTransfer)
{
	TSharedPtr<Private::FBuildTransferState> NewState = MakeShared<Private::FBuildTransferState>();

	{
		FReadScopeLock _(BuildTransfer.State->Lock);
		NewState->Type = BuildTransfer.State->Type;
		NewState->BuildId = BuildTransfer.State->BuildId;
		NewState->Name = BuildTransfer.State->Name;
		NewState->Namespace = BuildTransfer.State->Namespace;
		NewState->Bucket = BuildTransfer.State->Bucket;
		NewState->Destination = BuildTransfer.State->Destination;
		NewState->IncludeFilter = BuildTransfer.State->IncludeFilter;
		NewState->ExcludeFilter = BuildTransfer.State->ExcludeFilter;
		NewState->DownloadSpecJSONContents = BuildTransfer.State->DownloadSpecJSONContents;
		NewState->PartIds = BuildTransfer.State->PartIds;
		NewState->PartNames = BuildTransfer.State->PartNames;
		NewState->RootDir = BuildTransfer.State->RootDir;
		NewState->EngineDir = BuildTransfer.State->EngineDir;
		NewState->ProjectFilePath = BuildTransfer.State->ProjectFilePath;
		NewState->TargetPlatformName = BuildTransfer.State->TargetPlatformName;
		NewState->HostOverride = BuildTransfer.State->HostOverride;
		NewState->RequestFlags = BuildTransfer.State->RequestFlags;
		NewState->StartTime = FPlatformTime::Seconds();
	}
	
	BuildTransferThreadStates.Enqueue(NewState.ToSharedRef());
	KickBuildTransferThread();

	FBuildTransfer NewTransfer;
	NewTransfer.State = NewState;
	return NewTransfer;
}

FBuildServiceInstance::FBuildTransfer::EType
FBuildServiceInstance::FBuildTransfer::GetType() const
{
	if (!State)
	{
		return EType::Count;
	}
	return State->Type;
}

FString FBuildServiceInstance::FBuildTransfer::GetDestination() const
{
	if (!State)
	{
		return FString();
	}
	return State->Destination;
}

FString FBuildServiceInstance::FBuildRecord::GetCommitIdentifier() const
{
	FString CommitIdentifier;
	if (FCbFieldView ChangelistField = Metadata["changelist"]; ChangelistField.HasValue() && !ChangelistField.HasError())
	{
		if (ChangelistField.IsString())
		{
			CommitIdentifier = FUTF8ToTCHAR(ChangelistField.AsString());
		}
		else if (ChangelistField.IsInteger())
		{
			CommitIdentifier = *WriteToString<64>(ChangelistField.AsUInt64());
		}
		else if (ChangelistField.IsFloat())
		{
			CommitIdentifier = *WriteToString<64>((uint64)ChangelistField.AsDouble());
		}
	}
	else if (FCbFieldView CommitField = Metadata["commit"]; CommitField.HasValue() && !CommitField.HasError())
	{
		if (CommitField.IsString())
		{
			CommitIdentifier = FUTF8ToTCHAR(CommitField.AsString());
		}
		else if (CommitField.IsInteger())
		{
			CommitIdentifier = *WriteToString<64>(CommitField.AsUInt64());
		}
		else if (CommitField.IsFloat())
		{
			CommitIdentifier = *WriteToString<64>((uint64)CommitField.AsDouble());
		}
	}

	return CommitIdentifier;
}

FString FBuildServiceInstance::FBuildRecord::GetCookPlatform() const
{
	FString CookPlatform;
	if (FCbFieldView CookPlatformField = Metadata["cookPlatform"]; CookPlatformField.HasValue() && !CookPlatformField.HasError())
	{
		CookPlatform = *WriteToString<64>(CookPlatformField.AsString());
	}

	return CookPlatform;
}

FString
FBuildServiceInstance::FBuildTransfer::GetDescription() const
{
	if (!State)
	{
		return TEXT("null");
	}

	FReadScopeLock _(State->Lock);
	if (State->NextOutputLineIndex == 0)
	{
		return FString();
	}
	return State->RecentOutputLines[State->RecentOutputLines.GetPreviousIndex(State->NextOutputLineIndex)];
}

FString
FBuildServiceInstance::FBuildTransfer::GetRecentOutput() const
{
	if (!State)
	{
		return TEXT("null");
	}

	FReadScopeLock _(State->Lock);
	return State->GetRecentOutput();
}

FBuildServiceInstance::EBuildTransferStatus
FBuildServiceInstance::FBuildTransfer::GetStatus() const
{
	if (!State)
	{
		return EBuildTransferStatus::Invalid;
	}

	FReadScopeLock _(State->Lock);
	return State->Status;
}

FString
FBuildServiceInstance::FBuildTransfer::GetLogFilename() const
{
	if (!State)
	{
		return FString();
	}

	FReadScopeLock _(State->Lock);
	return State->GetLogFilename();
}

FMonotonicTimeSpan
FBuildServiceInstance::FBuildTransfer::GetTransferDuration() const
{
	if (!State)
	{
		return FMonotonicTimeSpan::Zero();
	}

	FReadScopeLock _(State->Lock);
	if (State->Status == EBuildTransferStatus::Canceled ||
		State->Status == EBuildTransferStatus::Failed ||
		State->Status == EBuildTransferStatus::Succeeded)
	{
		return State->TransferEndTime - State->TransferStartTime;
	}
	else if (State->Status == EBuildTransferStatus::Active)
	{
		return FMonotonicTimePoint::Now() - State->TransferStartTime;
	}
	return FMonotonicTimeSpan::Zero();
}

bool
FBuildServiceInstance::FBuildTransfer::GetCurrentProgress(FString& OutLabel, FString& OutDetail, float& OutPercent) const
{
	if (!State)
	{
		return false;
	}

	FReadScopeLock _(State->Lock);
	if (State->ProgressState.IsEmpty())
	{
		if (State->NextOutputLineIndex == 0)
		{
			OutLabel.Empty();
		}
		else
		{
			OutLabel = State->RecentOutputLines[State->RecentOutputLines.GetPreviousIndex(State->NextOutputLineIndex)];
		}
		if (State->NextOutputLineIndex > 0)
		{
			OutDetail = State->RecentOutputLines[State->NextOutputLineIndex - 1];
		}

		OutPercent = 0.f;
		return true;
	}
	OutLabel = State->ProgressState.Top().Label;
	OutDetail = State->ProgressState.Top().Detail;
	OutPercent = State->ProgressState.Top().Percent;
	return true;
}

bool
FBuildServiceInstance::FBuildTransfer::GetOverallProgress(FString& OutLabel, FString& OutDetail, float& OutPercent) const
{
	if (!State)
	{
		return false;
	}

	FReadScopeLock _(State->Lock);
	if (State->ProgressState.IsEmpty())
	{
		if (State->NextOutputLineIndex == 0)
		{
			OutLabel.Empty();
		}
		else
		{
			OutLabel = State->RecentOutputLines[State->RecentOutputLines.GetPreviousIndex(State->NextOutputLineIndex)];
		}
		if (State->NextOutputLineIndex > 0)
		{
			OutDetail = State->RecentOutputLines[State->NextOutputLineIndex - 1];
		}

		OutPercent = 0.f;
		return true;
	}
	OutLabel = State->ProgressState[0].Label;
	OutDetail = State->ProgressState[0].Detail;
	OutPercent = State->ProgressState[0].Percent;
	return true;
}

void
FBuildServiceInstance::FBuildTransfer::RequestCancel()
{
	if (!State)
	{
		return;
	}
	State->bCancelRequested.store(true);
	FWriteScopeLock _(State->Lock);
	if (State->Status == EBuildTransferStatus::Active)
	{
		State->TransferEndTime = FMonotonicTimePoint::Now();
	}
	State->Status = EBuildTransferStatus::Canceled;
}

FBuildServiceInstance::EConnectionState
FBuildServiceInstance::GetConnectionState() const
{
	return ConnectionState.load(std::memory_order_relaxed);
}

FAnsiStringView
FBuildServiceInstance::GetEffectiveDomain() const
{
	return EffectiveDomain;
}

void
FBuildServiceInstance::Initialize()
{
#if WITH_SSL
	// Load SSL module during HTTP module's StatupModule() to make sure module manager figures out the dependencies correctly
	// and doesn't unload SSL before unloading HTTP module at exit
	FSslModule::Get();
#endif
	ISocketSubsystem::Get();
	FDesktopPlatformModule::TryGet();
	ZenService = MakePimpl<FScopeZenService>();
}

bool
FBuildServiceInstance::AcquireAccessToken(EDesktopLoginInteractionLevel InteractionLevel)
{
	if (GetEffectiveDomain().StartsWith("http://localhost"))
	{
		UE_LOGF(LogBuildServiceInstance, Log, "Skipping authorization for connection to localhost.");
		return true;
	}

	LoginAttempts++;

	// Avoid spamming this if the service is down.
	if (FailedLoginAttempts > UE_BUILDSERVERINSTANCE_MAX_FAILED_LOGIN_ATTEMPTS)
	{
		UE_LOGF(LogBuildServiceInstance, Display,
			"OidcToken: Skipping token refresh due to failed login attempts (%d).", FailedLoginAttempts);
		return false;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(BuildServiceInstance_AcquireAccessToken);

	// In case many requests wants to update the token at the same time
	// get the current serial while we wait to take the CS.
	const uint32 WantsToUpdateTokenSerial = Access ? Access->GetSerial() : 0;

	FScopeLock Lock(&AccessCs);

	// If the token was updated while we waited to take the lock, then it should now be valid.
	if (Access && Access->GetSerial() > WantsToUpdateTokenSerial)
	{
		UE_LOGF(LogBuildServiceInstance, Display,
			"OidcToken: Skipping token refresh due to token serial change.");
		return true;
	}

	FString AccessTokenString;
	FDateTime TokenExpiresAt;
	bool bWasInteractiveLogin = false;

	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::TryGet();
	if (DesktopPlatform && DesktopPlatform->GetOidcAccessTokenWithRemoteConfig(FPaths::RootDir(), *WriteToString<64>(EffectiveDomain), InteractionLevel, GWarn, AccessTokenString, TokenExpiresAt, bWasInteractiveLogin))
	{
		if (bWasInteractiveLogin)
		{
			InteractiveLoginAttempts++;
		}

		const double ExpiryTimeSeconds = (TokenExpiresAt - FDateTime::UtcNow()).GetTotalSeconds();
		UE_LOGF(LogBuildServiceInstance, Display,
			"OidcToken: Logged in to HTTP Build services. Expires at %ls which is in %.0f seconds.",
			*TokenExpiresAt.ToString(), ExpiryTimeSeconds);
		SetAccessTokenAndUnlock(Lock, AccessTokenString, ExpiryTimeSeconds);
		RecordAcquireAccessTokenEvent(true);
		
		return true;
	}
	else if (DesktopPlatform)
	{
		UE_LOGF(LogBuildServiceInstance, Warning, "OidcToken: Failed to log in to HTTP services.");
		RecordAcquireAccessTokenEvent(false, TEXT("OidcToken: Failed to log in to HTTP services."));
		FailedLoginAttempts++;
		return false;
	}
	else
	{
		UE_LOGF(LogBuildServiceInstance, Warning, "OidcToken: Use of OAuthProviderIdentifier requires that the target depend on DesktopPlatform.");
		RecordAcquireAccessTokenEvent(false, TEXT("OidcToken: Use of OAuthProviderIdentifier requires that the target depend on DesktopPlatform."));
		FailedLoginAttempts++;
		return false;
	}
}

void
FBuildServiceInstance::SetAccessTokenAndUnlock(FScopeLock& Lock, FStringView Token, double RefreshDelay)
{
	// Cache the expired refresh handle.
	FTSTicker::FDelegateHandle ExpiredRefreshAccessTokenHandle = MoveTemp(RefreshAccessTokenHandle);
	RefreshAccessTokenHandle.Reset();

	if (!Access)
	{
		Access = MakeUnique<Private::FAccessToken>();
	}
	Access->SetToken(Settings.GetAuthScheme(), Token);

	constexpr double RefreshGracePeriod = 20.0f;
	if (RefreshDelay > RefreshGracePeriod)
	{
		// Schedule a refresh of the token ahead of expiry time (this will not work in commandlets)
		if (!IsRunningCommandlet())
		{
			UE_LOGF(LogBuildServiceInstance, Display,
				"OidcToken: Scheduling token refresh in %.0f seconds.",
				float(FMath::Min(RefreshDelay - RefreshGracePeriod, MAX_flt)));

			RefreshAccessTokenHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(
				[this](float DeltaTime)
				{
					UE_LOGF(LogBuildServiceInstance, Display,
						"OidcToken: Executing token refresh (no interaction).");
					AcquireAccessToken(EDesktopLoginInteractionLevel::None);
					return false;
				}
			), float(FMath::Min(RefreshDelay - RefreshGracePeriod, MAX_flt)));
		}

		// Schedule a forced refresh of the token when the scheduled refresh is starved or unavailable.
		RefreshAccessTokenTime = FPlatformTime::Seconds() + RefreshDelay - RefreshGracePeriod * 0.5f;
	}
	else
	{
		RefreshAccessTokenTime = 0.0;
	}

	// Reset failed login attempts, the service is indeed alive.
	FailedLoginAttempts = 0;

	// Unlock the critical section before attempting to remove the expired refresh handle.
	// The associated ticker delegate could already be executing, which could cause a
	// hang in RemoveTicker when the critical section is locked.
	Lock.Unlock();
	if (ExpiredRefreshAccessTokenHandle.IsValid())
	{
		FTSTicker::RemoveTicker(MoveTemp(ExpiredRefreshAccessTokenHandle));
	}
}

FString
FBuildServiceInstance::GetAccessToken() const
{
	TAnsiStringBuilder<128> AccessTokenBuilder;
	if (Access.IsValid())
	{
		AccessTokenBuilder << *Access;
	}
	return FString(AccessTokenBuilder);
}

FString
FBuildServiceInstance::AddZenUtilityBuildServerArguments(const TCHAR* InCommandLineArgs) const
{
	return FString::Printf(TEXT("%s --host %hs --access-token \"%s\""),
			InCommandLineArgs, *EffectiveDomain, *GetAccessToken());
}

bool
FBuildServiceInstance::InvokeZenUtilitySync(FStringView InCommandLineArgs, FString* OutputText)
{
	FString ZenUtilityPath = UE::Zen::GetLocalInstallUtilityPath();

	FString CommandLineArgs(InCommandLineArgs);

	if (OutputText)
	{
		OutputText->Empty();
	}

	FString AbsoluteUtilityPath = FPaths::ConvertRelativePathToFull(ZenUtilityPath);
	FMonitoredProcess MonitoredUtilityProcess(AbsoluteUtilityPath, *CommandLineArgs, FPaths::GetPath(AbsoluteUtilityPath), true);
	if (!MonitoredUtilityProcess.Launch())
	{
		const FString MessageString = FString::Printf( TEXT("Failed to launch zen utility to gather build data in path '%s'."), *AbsoluteUtilityPath );
		UE_LOGF(LogBuildServiceInstance, Warning, "%ls", *MessageString);
		RecordZenServerEvent(false, MessageString);
		
		return false;
	}

	const uint64 StartTime = FPlatformTime::Cycles64();
	while (MonitoredUtilityProcess.Update())
	{
		double Duration = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);
		if (Duration > 120.0)
		{
			MonitoredUtilityProcess.Cancel(true);
			const FString MessageString = FString::Printf( TEXT("Cancelled launch of zen utility for gathering build data: '%s' due to timeout."), *AbsoluteUtilityPath);
			UE_LOGF(LogBuildServiceInstance, Warning, "%ls", *MessageString);
			RecordZenServerEvent(false, MessageString);
			
			// Wait for execution to be terminated
			while (MonitoredUtilityProcess.Update())
			{
				Duration = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);
				if (Duration > 15.0)
				{
					UE_LOGF(LogBuildServiceInstance, Warning, "Cancelled launch of zen utility for gathering build data: '%ls'. Failed waiting for termination.", *AbsoluteUtilityPath);
					break;
				}
				FPlatformProcess::Sleep(0.2f);
			}

			FString OutputString = MonitoredUtilityProcess.GetFullOutputWithoutDelegate();
			UE_LOGF(LogBuildServiceInstance, Warning, "Launch of zen utility for gathering build data: '%ls' failed. Output: '%ls'", *AbsoluteUtilityPath, *OutputString);
			return false;
		}
		FPlatformProcess::Sleep(0.1f);
	}

	FString OutputString = MonitoredUtilityProcess.GetFullOutputWithoutDelegate();
	if (MonitoredUtilityProcess.GetReturnCode() != 0)
	{
		UE_LOGF(LogBuildServiceInstance, Warning, "Unexpected return code after launch of zen utility for gathering build data: '%ls' (%d). Output: '%ls'", *AbsoluteUtilityPath, MonitoredUtilityProcess.GetReturnCode(), *OutputString);
		RecordZenServerEvent(false, TEXT("Unexpected return code after launch of zen utility for gathering build data"));
		return false;
	}

	if (OutputText)
	{
		*OutputText = OutputString;
	}

	return true;
}

void
FBuildServiceInstance::InvokeZenUtility(FStringView InCommandLineArgs, FPreZenUtilityInvocation&& PreInvocation, FOnZenUtilityInvocationComplete&& OnComplete)
{
	UE::Tasks::Launch(UE_SOURCE_LOCATION,
	[this, InCommandLineArgs = FString(InCommandLineArgs), PreInvocation = MoveTemp(PreInvocation), OnComplete = MoveTemp(OnComplete)]() mutable
	{
		if (PreInvocation)
		{
			PreInvocation(InCommandLineArgs);
		}

		bool bSuccessfulInvocation = InvokeZenUtilitySync(InCommandLineArgs);

		if (OnComplete)
		{
			OnComplete(bSuccessfulInvocation);
		}
	});
}

void
FBuildServiceInstance::CallOnRefreshNamespacesAndBucketsComplete(FOnRefreshNamespacesAndBucketsComplete&& InOnRefreshNamespacesAndBucketsComplete)
{
	if (InOnRefreshNamespacesAndBucketsComplete)
	{
		InOnRefreshNamespacesAndBucketsComplete();
	}

	RefreshNamespacesAndBucketsComplete.Broadcast();
}

void
FBuildServiceInstance::KickBuildTransferThread()
{
	if (!bBuildTransferThreadStarting.load(std::memory_order_relaxed) && !bBuildTransferThreadStarting.exchange(true, std::memory_order_relaxed))
	{
		BuildTransferThread = FThread(TEXT("BuildTransfer"), [this] { BuildTransferThreadLoop(); }, 128 * 1024);
	}
}

void
FBuildServiceInstance::BuildTransferThreadLoop()
{
	while (!BuildTransferThreadStates.IsEmpty() || !bBuildTransferThreadStopping.load(std::memory_order_relaxed))
	{
		TOptional<TSharedRef<Private::FBuildTransferState>> OptionalState = BuildTransferThreadStates.Dequeue();
		if (!OptionalState)
		{
			FPlatformProcess::Sleep(0.2f);
			continue;
		}
		TSharedRef<Private::FBuildTransferState> State(OptionalState.GetValue());

		if (State->bCancelRequested.load(std::memory_order_relaxed))
		{
			FWriteScopeLock _(State->Lock);
			if (State->Status == EBuildTransferStatus::Active)
			{
				State->TransferEndTime = FMonotonicTimePoint::Now();
			}
			State->Status = EBuildTransferStatus::Canceled;
			continue;
		}

 		FString ZenUtilityPath = FPaths::ConvertRelativePathToFull(UE::Zen::GetLocalInstallUtilityPath());
 		FString OidcTokenExecutableFilename = FPaths::ConvertRelativePathToFull(FDesktopPlatformModule::TryGet()->GetOidcTokenExecutableFilename(FPaths::RootDir()));

 		FString DownloadSpecTempFile;
 		ON_SCOPE_EXIT
 		{
 			if (!DownloadSpecTempFile.IsEmpty())
 			{
				IFileManager::Get().Delete(*DownloadSpecTempFile);
 			}
 		};
 		FString DestinationDirectory;
		TStringBuilder<256> CommandLineBuilder;
		switch (State->Type)
		{
		case FBuildTransfer::EType::Files:
			DestinationDirectory = FPaths::ConvertRelativePathToFull(State->Destination);
			CommandLineBuilder.Append(FString::Printf(TEXT("builds download --log-progress --host %hs --local-path \"%s\" --namespace %s --bucket %s --build-id %s%s%s --oidctoken-exe-path \"%s\""),
				*EffectiveDomain, *DestinationDirectory, *State->Namespace, *State->Bucket,
				*WriteToString<64>(State->BuildId),
				State->IncludeFilter.IsEmpty() ? TEXT("") : *WriteToString<64>(TEXT(" --wildcard \""), State->IncludeFilter, TEXT("\"")),
				State->ExcludeFilter.IsEmpty() ? TEXT("") : *WriteToString<64>(TEXT(" --exclude-wildcard \""), State->ExcludeFilter, TEXT("\"")),
				*OidcTokenExecutableFilename));

				if (!State->HostOverride.IsEmpty())
				{
					CommandLineBuilder.Appendf(TEXT(" --override-host \"%s\""), *State->HostOverride);
				}

				if (!State->PartIds.IsEmpty())
				{
					CommandLineBuilder.Appendf(TEXT(" --build-part-id \"%s\""), *FString::Join(State->PartIds, TEXT(",")));
				}

				if (!State->PartNames.IsEmpty())
				{
					CommandLineBuilder.Appendf(TEXT(" --build-part-name \"%s\""), *FString::Join(State->PartNames, TEXT(",")));
				}

				if (!State->DownloadSpecJSONContents.IsEmpty())
				{
					DownloadSpecTempFile = FPaths::CreateTempFilename(FPlatformProcess::UserTempDir(), TEXT("download"), TEXT(".json"));
					if (FFileHelper::SaveStringToFile(State->DownloadSpecJSONContents, *DownloadSpecTempFile))
					{
						CommandLineBuilder.Appendf(TEXT(" --download-spec-path \"%s\""), *DownloadSpecTempFile);
					}
					else
					{
						UE_LOGF(LogBuildServiceInstance, Warning, "Failed to write download specification file, full build will be downloaded.");
						DownloadSpecTempFile.Empty();
					}
				}

				if (EnumHasAnyFlags(State->RequestFlags, EBuildTransferRequestFlags::Scavenge))
				{
					CommandLineBuilder.Append(TEXT(" --enable-scavenge"));
				}

				if (EnumHasAnyFlags(State->RequestFlags, EBuildTransferRequestFlags::Verify))
				{
					CommandLineBuilder.Append(TEXT(" --verify"));
				}

				if (EnumHasAnyFlags(State->RequestFlags, EBuildTransferRequestFlags::Append))
				{
					CommandLineBuilder.Append(TEXT(" --append"));
				}
			break;
		case FBuildTransfer::EType::Oplog:
			{
				FString DestinationProjectId, DestinationOplogId;
				FString SplitDelim(TEXT("/"));
				State->Destination.Split(SplitDelim, &DestinationProjectId, &DestinationOplogId);

				UE::Zen::FZenLocalServiceRunContext RunContext;
				uint16 LocalPort = 8558;
				if (UE::Zen::TryGetLocalServiceRunContext(RunContext))
				{
					if (!UE::Zen::IsLocalServiceRunning(*RunContext.GetDataPath(), &LocalPort))
					{
						UE::Zen::StartLocalService(RunContext);
						UE::Zen::IsLocalServiceRunning(*RunContext.GetDataPath(), &LocalPort);
					}
				}

				FString RootDir;
				FString EngineDir;
				FString ProjectFilePath;

				// Try to derive the project file path from destination project in zenserver if possible
				FString ProjectInfoText;
				if (InvokeZenUtilitySync(FString::Printf(TEXT("project-info --project \"%s\""),
					*DestinationProjectId), &ProjectInfoText))
				{
					int32 StartOfJson = INDEX_NONE;
					if (ProjectInfoText.FindChar(TEXT('{'), StartOfJson))
					{
						ProjectInfoText.RightChopInline(StartOfJson);

						TSharedPtr<FJsonObject> ProjectInfoObject;
						TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(ProjectInfoText);
						if (FJsonSerializer::Deserialize(Reader, ProjectInfoObject) && ProjectInfoObject.IsValid())
						{
							ProjectInfoObject->TryGetStringField(TEXT("root"), RootDir);
							ProjectInfoObject->TryGetStringField(TEXT("engine"), EngineDir);
							ProjectInfoObject->TryGetStringField(TEXT("projectfile"), ProjectFilePath);
						}
					}
				}

				// Only use the specified root/engine/project file path if we didn't determine one from the destination project in zenserver
				if (RootDir.IsEmpty())
				{
					RootDir = State->RootDir;
				}
				if (EngineDir.IsEmpty())
				{
					EngineDir = State->EngineDir;
				}
				if (ProjectFilePath.IsEmpty())
				{
					ProjectFilePath = State->ProjectFilePath;
				}

				if (!ProjectFilePath.IsEmpty())
				{
					DestinationDirectory = FPaths::Combine(FPaths::GetPath(ProjectFilePath),
														TEXT("Saved"),
														TEXT("Cooked"),
														DestinationOplogId);

					FString OplogLifetimeMarkerPath = FPaths::Combine(DestinationDirectory,
														TEXT("ue.projectstore"));

					FString HostAuthJson;
					UE::Zen::GetLocalServiceHostAuth(HostAuthJson);

					Zen::FZenProjectStoreWriter ProjectStoreWriter(*OplogLifetimeMarkerPath);
					ProjectStoreWriter.Write(TEXT("[::1]"), LocalPort, HostAuthJson, true, DestinationProjectId, DestinationOplogId, State->TargetPlatformName);

					FPaths::NormalizeDirectoryName(EngineDir);
					FString ProjectDir = FPaths::GetPath(ProjectFilePath);
					FPaths::NormalizeDirectoryName(ProjectDir);
					FString ProjectPath = ProjectFilePath;
					FPaths::NormalizeFilename(ProjectPath);

					IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
					FString AbsServerRoot = PlatformFile.ConvertToAbsolutePathForExternalAppForRead(*RootDir);
					FString AbsEngineDir = PlatformFile.ConvertToAbsolutePathForExternalAppForRead(*EngineDir);
					FString AbsProjectDir = PlatformFile.ConvertToAbsolutePathForExternalAppForRead(*ProjectDir);
					FString AbsProjectFilePath = PlatformFile.ConvertToAbsolutePathForExternalAppForRead(*ProjectPath);

					InvokeZenUtilitySync(FString::Printf(TEXT("project-create --project \"%s\" --rootdir \"%s\" --enginedir \"%s\" --projectdir \"%s\" --projectfile \"%s\""),
						*DestinationProjectId, *AbsServerRoot, *AbsEngineDir, *AbsProjectDir, *AbsProjectFilePath));

					InvokeZenUtilitySync(FString::Printf(TEXT("oplog-create --project \"%s\" --oplog \"%s\" --gcpath \"%s\""),
						*DestinationProjectId, *DestinationOplogId, *OplogLifetimeMarkerPath));
				}
				else
				{
					InvokeZenUtilitySync(FString::Printf(TEXT("project-create --project \"%s\""),
						*DestinationProjectId));

					InvokeZenUtilitySync(FString::Printf(TEXT("oplog-create --project \"%s\" --oplog \"%s\""),
						*DestinationProjectId, *DestinationOplogId));
				}
				CommandLineBuilder.Append(FString::Printf(TEXT("oplog-import %s %s --builds %hs --namespace %s --bucket %s --builds-id %s --oidctoken-exe-path \"%s\""),
					*DestinationProjectId, *DestinationOplogId, *EffectiveDomain,
					*State->Namespace, *State->Bucket,
					*WriteToString<64>(State->BuildId), *OidcTokenExecutableFilename));

				if (!State->HostOverride.IsEmpty())
				{
					CommandLineBuilder.Appendf(TEXT(" --builds-override-host \"%s\""), *State->HostOverride);
				}
			}
			break;
		}

		if (EnumHasAnyFlags(State->RequestFlags, EBuildTransferRequestFlags::Clean))
		{
			CommandLineBuilder.Append(TEXT(" --clean"));
		}

		if (EnumHasAnyFlags(State->RequestFlags, EBuildTransferRequestFlags::Force))
		{
			CommandLineBuilder.Append(TEXT(" --force"));
		}

		if (EnumHasAllFlags(State->RequestFlags, EBuildTransferRequestFlags::BoostWorkers))
		{
			CommandLineBuilder.Append(TEXT(" --boost-workers"));
		}
		else if (EnumHasAllFlags(State->RequestFlags, EBuildTransferRequestFlags::BoostWorkerCount))
		{
			CommandLineBuilder.Append(TEXT(" --boost-worker-count"));
		}
		else if (EnumHasAllFlags(State->RequestFlags, EBuildTransferRequestFlags::BoostWorkerMemory))
		{
			CommandLineBuilder.Append(TEXT(" --boost-worker-memory"));
		}

		UE_LOGF(LogBuildServiceInstance, Display, "Running: \"%ls\" %ls.", *ZenUtilityPath, *CommandLineBuilder);
		FMonitoredProcess MonitoredUtilityProcess(ZenUtilityPath, *CommandLineBuilder, FPaths::GetPath(ZenUtilityPath), true);
		MonitoredUtilityProcess.OnOutput().BindSPLambda(State, [State](FString Output)
		{
			FReadScopeLock _(State->Lock);
			if (State->OutputLogFile)
			{
				State->OutputLogFile->Logf(TEXT("%s"), *Output);	// Log should be unaltered output
				State->OutputLogFile->Flush();
			}
			if (!Output.IsEmpty())	// Skip empty lines
			{
				if (Output.StartsWith(TEXT("@progress push")))
				{
					Output = Output.RightChop(15);
					Private::FProgressState NewState;
					NewState.Label = Output;
					State->ProgressState.Push(MoveTemp(NewState));
					return;
				}
				else if (Output.StartsWith(TEXT("@progress pop")))
				{
					State->ProgressState.Pop();
					return;
				}
				else if (Output.StartsWith(TEXT("@progress")))
				{
					if (State->ProgressState.IsEmpty())
					{
						Private::FProgressState NewState;
						State->ProgressState.Push(MoveTemp(NewState));
					}
					Private::FProgressState& CurrentState = State->ProgressState.Top();
					Output = Output.RightChop(10);
					if (Output.EndsWith(TEXT("%")))
					{
						FString PossiblePercentText = Output.LeftChop(1);
						if (!PossiblePercentText.IsEmpty())
						{
							float Percent = FCString::Atof(*PossiblePercentText);
							if ((Percent != 0.0f) || (PossiblePercentText.Len() == 1 && PossiblePercentText[0] == TCHAR('0')))
							{
								CurrentState.Percent = Percent;
								return; // Do not print the percent progress alone
							}
						}
					}
					CurrentState.Detail = Output;
					Output = FString::Printf(TEXT("[%s %d%%] %s"), *CurrentState.Label, (int32)CurrentState.Percent, *Output);
					// Deliberate fall-through so that the text is printed;
				}
				State->RecentOutputLines[State->NextOutputLineIndex++] = MoveTemp(Output);
			}
		});

		{
			FWriteScopeLock _(State->Lock);
			FString LogFilename = State->GetLogFilename();
			FOutputDeviceFile::CreateBackupCopy(*LogFilename);
			State->OutputLogFile = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*LogFilename, FILEWRITE_Silent | FILEWRITE_AllowRead));
			State->OutputLogFile->Logf(TEXT("Running: \"%s\" %s"), *ZenUtilityPath, *CommandLineBuilder);
			State->OutputLogFile->Flush();
			State->Status = EBuildTransferStatus::Active;
			State->TransferStartTime = FMonotonicTimePoint::Now();
		}

		if (!MonitoredUtilityProcess.Launch())
		{
			UE_LOGF(LogBuildServiceInstance, Warning, "Failed to launch zen utility to download build.");
			FWriteScopeLock _(State->Lock);
			State->TransferEndTime = FMonotonicTimePoint::Now();
			State->Status = EBuildTransferStatus::Failed;
			State->OutputLogFile->Logf(TEXT("Failed to launch."));
			State->OutputLogFile.Reset();
			continue;
		}

		const uint64 StartTime = FPlatformTime::Cycles64();
		while (MonitoredUtilityProcess.Update())
		{
			double Duration = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);
			if (bBuildTransferThreadStopping.load(std::memory_order_relaxed) || State->bCancelRequested.load(std::memory_order_relaxed))
			{
				// Send Ctrl+Break / SIGINT to allow graceful shutdown
				bool bSentSignal = SendSoftTerminationSignal(MonitoredUtilityProcess.GetProcessHandle());
				if (bSentSignal)
				{
					UE_LOGF(LogBuildServiceInstance, Display,
						"Sent Ctrl+Break to zen utility for build data transfer: '%ls'. Waiting up to %.0f seconds for graceful shutdown.",
						*ZenUtilityPath, GracefulTerminationTimeoutSeconds);
				}
				else
				{
					UE_LOGF(LogBuildServiceInstance, Warning,
						"Failed to send Ctrl+Break to zen utility: '%ls'. Falling back to hard termination.",
						*ZenUtilityPath);
					MonitoredUtilityProcess.Cancel(true);
				}

				// Wait for graceful exit (or immediate exit if hard-terminated above)
				const uint64 GraceStartTime = FPlatformTime::Cycles64();
				bool bExitedGracefully = false;
				while (MonitoredUtilityProcess.Update())
				{
					double GraceDuration = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - GraceStartTime);
					if (GraceDuration > GracefulTerminationTimeoutSeconds)
					{
						break;
					}
					FPlatformProcess::Sleep(0.2f);
				}

				if (!MonitoredUtilityProcess.Update())
				{
					bExitedGracefully = true;
				}

				// If still running after grace period, hard terminate
				if (!bExitedGracefully && bSentSignal)
				{
					UE_LOGF(LogBuildServiceInstance, Warning,
						"Zen utility '%ls' did not respond to Ctrl+Break signal within %.0f second grace period. Hard-terminating.",
						*ZenUtilityPath, GracefulTerminationTimeoutSeconds);
					MonitoredUtilityProcess.Cancel(true);

					const uint64 HardKillStartTime = FPlatformTime::Cycles64();
					while (MonitoredUtilityProcess.Update())
					{
						double HardKillDuration = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - HardKillStartTime);
						if (HardKillDuration > 10.0)
						{
							UE_LOGF(LogBuildServiceInstance, Warning,
								"Zen utility '%ls' did not terminate after hard kill within 10 seconds.", *ZenUtilityPath);
							break;
						}
						FPlatformProcess::Sleep(0.2f);
					}
				}

				FWriteScopeLock _(State->Lock);
				State->TransferEndTime = FMonotonicTimePoint::Now();
				State->Status = EBuildTransferStatus::Canceled;
				double DurationSeconds = (State->TransferEndTime - State->TransferStartTime).ToSeconds();
				UE_LOGF(LogBuildServiceInstance, Display, "Transfer canceled after %.1f minutes.", DurationSeconds / 60.0f);
				State->OutputLogFile->Logf(TEXT("Transfer canceled after %.1f minutes."), DurationSeconds / 60.0f);
				State->OutputLogFile.Reset();
				RecordTransferStateEvent(State);

				break;
			}
			FPlatformProcess::Sleep(0.1f);
		}

		if (!State->bCancelRequested.load(std::memory_order_relaxed))
		{
			if (MonitoredUtilityProcess.GetReturnCode() == 0)
			{
				FWriteScopeLock _(State->Lock);
				State->ReturnCode = MonitoredUtilityProcess.GetReturnCode();
				State->TransferEndTime = FMonotonicTimePoint::Now();
				State->Status = EBuildTransferStatus::Succeeded;
				double DurationSeconds = (State->TransferEndTime - State->TransferStartTime).ToSeconds();
				UE_LOGF(LogBuildServiceInstance, Display, "Transfer completed successfully after %.1f minutes.", DurationSeconds/60.0f);
				State->OutputLogFile->Logf(TEXT("Transfer completed successfully after %.1f minutes."), DurationSeconds/60.0f);
				State->OutputLogFile.Reset();
				RecordTransferStateEvent(State);
			}
			else
			{
				FWriteScopeLock _(State->Lock);
				UE_LOGF(LogBuildServiceInstance, Warning, "Unexpected return code after launch of zen utility for downloading build data: '%ls' (%d). Output: '%ls'", *ZenUtilityPath, MonitoredUtilityProcess.GetReturnCode(), *State->GetRecentOutput());
				State->ReturnCode = MonitoredUtilityProcess.GetReturnCode();
				State->TransferEndTime = FMonotonicTimePoint::Now();
				State->Status = EBuildTransferStatus::Failed;
				double DurationSeconds = (State->TransferEndTime - State->TransferStartTime).ToSeconds();
				State->OutputLogFile->Logf(TEXT("Unexpected return code (%d) from transfer after %.1f minutes"), State->ReturnCode, DurationSeconds/60.0f);
				State->OutputLogFile.Reset();
				RecordTransferStateEvent(State);
			}
		}

		bool bCleanDestination = false;
		{
			FReadScopeLock _(State->Lock);
			bCleanDestination = (State->Status == EBuildTransferStatus::Succeeded) &&
				(State->Type == FBuildTransfer::EType::Oplog) &&
				!DestinationDirectory.IsEmpty();
		}

		if (bCleanDestination)
		{
			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
			PlatformFile.IterateDirectory(*DestinationDirectory,
				[&PlatformFile](const TCHAR* FoundFullPath, bool bDirectory)
				{
					if (bDirectory)
					{
						PlatformFile.DeleteDirectoryRecursively(FoundFullPath);
					}
					else
					{
						if (FPathViews::GetCleanFilename(FoundFullPath) != TEXT("ue.projectstore"))
						{
							PlatformFile.DeleteFile(FoundFullPath);
						}
					}
					return true;
				});
		}
	}
}

void
FBuildServiceInstance::RecordAcquireAccessTokenEvent(bool Succeeded, const FString& MessageString)
{
	// Generate and send Telemetry Event for accessing the Access Token
	if (FStudioTelemetry::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> Attributes;
		const int SchemaVersion = 1;
		Attributes.Emplace(TEXT("SchemaVersion"), SchemaVersion);
		Attributes.Emplace(TEXT("Host"), Settings.GetHost());
		Attributes.Emplace(TEXT("Outcome"), Succeeded ? TEXT("Succeeded") : TEXT("Failed"));
		Attributes.Emplace(TEXT("Message"), MessageString);
		Attributes.Emplace(TEXT("FailedAttempts"), FailedLoginAttempts);
		FStudioTelemetry::Get().RecordEvent(TEXT("Core.BuildService.AcquireAccessToken"), Attributes);
		FStudioTelemetry::Get().FlushEvents();
	}
}

void FBuildServiceInstance::RecordConnectionEvent(bool Succeeded, const FString& MessageString)
{
	// Generate and send Telemetry Event for connection
	if (FStudioTelemetry::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> Attributes;
		const int SchemaVersion = 1;
		Attributes.Emplace(TEXT("SchemaVersion"), SchemaVersion);
		Attributes.Emplace(TEXT("Host"), Settings.GetHost());
		Attributes.Emplace(TEXT("Outcome"), Succeeded ? TEXT("Succeeded") : TEXT("Failed"));
		Attributes.Emplace(TEXT("Message"), MessageString);
		FStudioTelemetry::Get().RecordEvent(TEXT("Core.BuildService.Connect"), Attributes);
		FStudioTelemetry::Get().FlushEvents();
	}
}

void
FBuildServiceInstance::RecordZenServerEvent(bool Succeeded, const FString& MessageString)
{
	// Generate and send Telemetry Event for interactions with the Zen Server
	if (FStudioTelemetry::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> Attributes;
		const int SchemaVersion = 1;
		Attributes.Emplace(TEXT("SchemaVersion"), SchemaVersion);
		Attributes.Emplace(TEXT("Host"), Settings.GetHost());
		Attributes.Emplace(TEXT("Outcome"), Succeeded ? TEXT("Succeeded") : TEXT("Failed"));
		Attributes.Emplace(TEXT("Message"), MessageString);
		FStudioTelemetry::Get().RecordEvent(TEXT("Core.BuildService.Zen"), Attributes);
		FStudioTelemetry::Get().FlushEvents();
	}
}

void
FBuildServiceInstance::RecordTransferStateEvent(TSharedRef<Private::FBuildTransferState> State)
{
	// Generate and send Telemetry Event for the transfer state
	if (FStudioTelemetry::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> Attributes;
		const int SchemaVersion = 1;

		FString Status;
		switch (State->Status)
		{
		case EBuildTransferStatus::Canceled:
		{
			Status = TEXT("Cancelled");
			break;
		}
		case EBuildTransferStatus::Failed:
		{
			Status = TEXT("Failed");
			break;
		}
		case EBuildTransferStatus::Succeeded:
		{
			Status = TEXT("Succeeded");
			break;
		}
		case EBuildTransferStatus::Queued:
		{
			Status = TEXT("Queued");
			break;
		}
		default:
		{
			Status = TEXT("Undefined");
			break;
		}
		}

		Attributes.Emplace(TEXT("SchemaVersion"), SchemaVersion);
		Attributes.Emplace(TEXT("Host"), Settings.GetHost());
		Attributes.Emplace(TEXT("Status"), Status);
		Attributes.Emplace(TEXT("Name"), State->Name);
		Attributes.Emplace(TEXT("Namespace"), State->Namespace);
		Attributes.Emplace(TEXT("Bucket"), State->Bucket);
		Attributes.Emplace(TEXT("ReturnCode"), State->ReturnCode);
		Attributes.Emplace(TEXT("Platform"), State->TargetPlatformName);
		Attributes.Emplace(TEXT("HostOverride"), State->HostOverride);
		Attributes.Emplace(TEXT("UsedDownloadSpec"), State->DownloadSpecJSONContents.IsEmpty() ? false : true);
		Attributes.Emplace(TEXT("RequestFlags"), static_cast<uint32>(State->RequestFlags));
		Attributes.Emplace(TEXT("Duration"), FPlatformTime::Seconds() - State->StartTime);

		FStudioTelemetry::Get().RecordEvent(TEXT("Core.BuildService.Transfer"), Attributes);
		FStudioTelemetry::Get().FlushEvents();
	}
}

} // namespace UE::Zen::Build
