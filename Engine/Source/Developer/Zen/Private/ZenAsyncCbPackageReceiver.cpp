// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZenAsyncCbPackageReceiver.h"

#include "Containers/UnrealString.h"
#include "Experimental/ZenServerInterface.h"
#include "Serialization/MemoryReader.h"
#include "ZenSerialization.h"

namespace UE::Zen
{

FAsyncCbPackageReceiver::FAsyncCbPackageReceiver(
	THttpUniquePtr<IHttpRequest>&& InRequest,
	Zen::FZenServiceInstance& InZenServiceInstance,
	FOnComplete&& InOnComplete,
	int InMaxAttempts,
	bool InUseSoftRecovery)
	: Request(MoveTemp(InRequest))
	, ZenServiceInstance(InZenServiceInstance)
	, BaseReceiver(Package, this)
	, OnCompleteCallback(MoveTemp(InOnComplete))
	, MaxAttempts(InMaxAttempts)
	, UseSoftRecovery(InUseSoftRecovery)
	, Attempt(0)
{
}

const IHttpResponse& FAsyncCbPackageReceiver::GetHttpResponse() { return *Response; }
const FCbPackage& FAsyncCbPackageReceiver::GetResponsePackage() { return Package; }

void FAsyncCbPackageReceiver::SendAsync()
{
	Request->SendAsync(this, Response);
}

IHttpReceiver* FAsyncCbPackageReceiver::OnCreate(IHttpResponse& LocalResponse)
{
	return &BaseReceiver;
}

IHttpReceiver* FAsyncCbPackageReceiver::OnComplete(IHttpResponse& LocalResponse)
{
	if ((++Attempt < MaxAttempts) && BaseReceiver.ShouldRetry(ZenServiceInstance, LocalResponse, UseSoftRecovery))
	{
		BaseReceiver.Reset();
		SendAsync();
		return nullptr;
	}

	Request.Reset();
	if (OnCompleteCallback)
	{
		// Ensuring that the OnComplete method is destroyed by the time we exit this method by moving it to a local scope variable
		FOnComplete LocalOnComplete = MoveTemp(OnCompleteCallback);

		// Calling LocalOnComplete may result in "this" being deleted, so no further access can happen to anything on "this"
		LocalOnComplete(this);
	}
	return nullptr;
}

FString FAsyncCbPackageReceiver::GetPayloadAsString() const
{
	return BaseReceiver.GetPayloadAsString(Response->GetContentType(), BaseReceiver.Body());
}

}
