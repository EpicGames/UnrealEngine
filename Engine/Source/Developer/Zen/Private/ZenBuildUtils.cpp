// Copyright Epic Games, Inc. All Rights Reserved.
#include "ZenBuildUtils.h"
#include "GenericPlatform/GenericPlatformHttp.h"
#include "String/ParseTokens.h"

namespace UE::Zen::Build
{

bool TryParseBuildReferenceFromUrl(FStringView Url, FBuildReference& OutReference)
{
	// Permissively allow any url with a path that ends with /<namespace>/<bucket>/<buildid>
	FString Path = FGenericPlatformHttp::GetUrlPath(Url);
	if (Path.IsEmpty())
	{
		return false;
	}

	TArray<FStringView> PathParts;
	UE::String::ParseTokens(Path,
		TCHAR('/'),
		[&PathParts](FStringView PathPart)
		{
			PathParts.Add(PathPart);
		}, UE::String::EParseTokensOptions::SkipEmpty);

	if (PathParts.Num() < 3)
	{
		return false;
	}

	FCbObjectId::ByteArray BuildIdBytes;
	FString BuildIdString(PathParts[PathParts.Num() - 1]);
	if (UE::String::HexToBytes(BuildIdString, BuildIdBytes) != sizeof(BuildIdBytes))
	{
		return false;
	}

	OutReference.Namespace = PathParts[PathParts.Num() - 3];
	OutReference.Bucket = PathParts[PathParts.Num() - 2];
	OutReference.BuildId = FCbObjectId(BuildIdBytes);
	return true;
}

} // namespace UE::Zen::Build
