// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZenCbPackageReceiver.h"

#include "Experimental/ZenServerInterface.h"
#include "Serialization/CompactBinaryPackage.h"
#include "Serialization/MemoryReader.h"
#include "ZenSerialization.h"

namespace UE::Zen
{

FCbPackageReceiver::FCbPackageReceiver(FCbPackage& OutPackage, IHttpReceiver* InNext)
	: Package(OutPackage)
	, Next(InNext)
{
}

void FCbPackageReceiver::Reset()
{
	BodyArray.Reset();
}

const FMemoryView FCbPackageReceiver::Body() const
{
	return MakeMemoryView(BodyArray);
}

bool FCbPackageReceiver::ShouldRetry(Zen::FZenServiceInstance& ZenServiceInstance, IHttpResponse& LocalResponse, bool UseSoftRecovery) const
{
	if (!UseSoftRecovery)
	{
		if (LocalResponse.GetErrorCode() == EHttpErrorCode::Unknown)
		{
			return TryRecover(ZenServiceInstance, UseSoftRecovery);
		}
	}

	if ((LocalResponse.GetErrorCode() == EHttpErrorCode::Connect) ||
		(LocalResponse.GetErrorCode() == EHttpErrorCode::TlsConnect) ||
		(LocalResponse.GetErrorCode() == EHttpErrorCode::TimedOut))
	{
		return TryRecover(ZenServiceInstance, UseSoftRecovery);
	}

	if (LocalResponse.GetStatusCode() == 404 && GetPayloadAsString(LocalResponse.GetContentType(), Body()).Contains("No suitable route found"))
	{
		// This is a special case if we happen to hit the port while zenserver is starting up - it is worth a retry
		return TryRecover(ZenServiceInstance, UseSoftRecovery);
	}

	return false;
}

bool FCbPackageReceiver::TryRecover(Zen::FZenServiceInstance& ZenServiceInstance, bool UseSoftRecovery) const
{
	if (ZenServiceInstance.IsServiceRunningLocally())
	{
		return ZenServiceInstance.TryRecovery(UseSoftRecovery ? Zen::FZenServiceInstance::ERecoveryMode::Soft : Zen::FZenServiceInstance::ERecoveryMode::Hard);
	}
	return true;
}

FString FCbPackageReceiver::GetPayloadAsString(EHttpMediaType ContentType, FMemoryView BodyView)
{
	switch (ContentType)
	{
	case EHttpMediaType::Text:
	case EHttpMediaType::Yaml:
	case EHttpMediaType::Json:
		if (BodyView.GetSize() > 0)
		{
			TStringBuilder<128> PayloadText;
			PayloadText << FAnsiStringView((const ANSICHAR*)BodyView.GetData(), (int)(BodyView.GetSize()));
			return *PayloadText;
		}
		return {};
	default:
	{
		TStringBuilder<128> SizeText;
		SizeText << "Payload (" << LexToString(ContentType) << "): " << BodyView.GetSize() << " bytes";
		return *SizeText;
	}
	}
}

IHttpReceiver* FCbPackageReceiver::OnCreate(IHttpResponse& Response)
{
	return &BodyReceiver;
}

IHttpReceiver* FCbPackageReceiver::OnComplete(IHttpResponse& Response)
{
	if (Response.GetErrorCode() == EHttpErrorCode::None)
	{
		EHttpMediaType ContentType = Response.GetContentType();
		switch (ContentType)
		{
			case EHttpMediaType::CbPackage:
			{
				const FMemoryView MemoryView = Body();
				{
					FMemoryReaderView Ar(MemoryView);
					if (Zen::Http::TryLoadCbPackage(Package, Ar))
					{
						BodyArray.Reset();
						return Next;
					}
				}
				FMemoryReaderView Ar(MemoryView);
				if (Package.TryLoad(Ar))
				{
					BodyArray.Reset();
				}
			}
			break;
		}
	}
	return Next;
}

}
