// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZenClient.h"
#include "Async/Async.h"
#include "Containers/StringView.h"
#include "HAL/Event.h"
#include "Math/UnrealMathUtility.h"
#include "Memory/MemoryView.h"
#include "Misc/StringBuilder.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Serialization/LargeMemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "IPAddress.h"
#include "Sockets.h"
#include "SocketSubsystem.h"

DEFINE_LOG_CATEGORY_STATIC(LogZenClient, Log, All);

TRACE_DECLARE_INT_COUNTER(ZenClient_StreamedBytes, TEXT("ZenClient/StreamedBytes"));
TRACE_DECLARE_FLOAT_COUNTER(ZenClient_StreamedSeconds, TEXT("ZenClient/StreamedSeconds"));

namespace UE::Zen {

////////////////////////////////////////////////////////////////////////////////
bool SendData(FSocket& Socket, FMemoryView Data)
{
	while (Data.IsEmpty() == false)
	{
		const int32 BytesToSend = (int32)FMath::Clamp<uint64>(Data.GetSize(), 0, MAX_int32);
		int32 BytesSent = 0;
		if (Socket.Send((const uint8*)Data.GetData(), BytesToSend, BytesSent) == false)
		{
			return false;
		}

		Data += BytesSent;
	}

	return true;
}

void CreateHttpUpgradeRequest(const TArray<FString>& Protocols, FAnsiStringBuilderBase& OutRequest)
{
	const FAnsiStringView Endpoint = ANSITEXTVIEW("/zen");

	OutRequest << ANSITEXTVIEW("GET ") << Endpoint << ANSITEXTVIEW(" HTTP/1.1\r\n");
	OutRequest << ANSITEXTVIEW("Upgrade: websocket\r\n");
	OutRequest << ANSITEXTVIEW("Connection: upgrade\r\n");
	OutRequest << ANSITEXTVIEW("Sec-WebSocket-Key: dGhlIHNhbXBsZSBub25jZQ==\r\n");

	if (Protocols.Num())
	{
		OutRequest << ANSITEXTVIEW("Sec-WebSocket-Protocol: ");

		for (int32 Idx = 0, Count = Protocols.Num(); Idx < Count; ++Idx)
		{
			if (Idx > 0)
			{
				OutRequest << ANSITEXTVIEW(" ");
			}

			FTCHARToUTF8 Utf8(*Protocols[Idx]);
			OutRequest << FAnsiStringView(Utf8.Get(), Utf8.Length());
		}
		
		OutRequest << ANSITEXTVIEW("\r\n");
	}

	OutRequest << ANSITEXTVIEW("\r\n");
}

bool ReceieveHttpUpgradeResponse(FSocket& Socket, FAnsiStringBuilderBase& Response)
{
	static constexpr int32 MaxLineSize = 1024;
	uint8 Buffer[MaxLineSize];

	for(;;)
	{
		int32 BytesRead = 0;
		Socket.Recv(Buffer, MaxLineSize, BytesRead, ESocketReceiveFlags::Peek);

		FAnsiStringView BufferView(reinterpret_cast<const ANSICHAR*>(Buffer), BytesRead);

		int32 LineEndIndex = 0;
		if (BufferView.FindChar('\n', LineEndIndex) && BytesRead > LineEndIndex)
		{
			check(BufferView[LineEndIndex - 1] == '\r');

			if (Socket.Recv(Buffer, LineEndIndex + 1, BytesRead, ESocketReceiveFlags::None) == false)
			{
				return false;
			}

			check(BytesRead == LineEndIndex + 1);
			
			FAnsiStringView Line = BufferView.Left(LineEndIndex - 1);
			if (Line.Len() == 0)
			{
				return true;
			}

			Response << Line << ANSITEXTVIEW("\n");
		}
	}
}

bool ValidateHandshake(FAnsiStringView UpgradeResponse, FString& OutReason)
{
	int32 NewLineIndex = 0;
	if (UpgradeResponse.FindChar('\n', NewLineIndex))
	{
		FAnsiStringView StatusLine = UpgradeResponse.Left(NewLineIndex);
		UpgradeResponse.RightChopInline(NewLineIndex + 1);

		if (StatusLine != ANSITEXTVIEW("HTTP/1.1 101 Switching Protocols"))
		{
			FUTF8ToTCHAR Str(StatusLine.GetData(), StatusLine.Len());
			OutReason = Str.Get(); 

			return false;
		}
	}

	//TODO: Validate accept hash and protocols

	return true;
}

////////////////////////////////////////////////////////////////////////////////
enum class EZenWebSocketMessageType : uint8
{
	Invalid,
	Notification,
	Request,
	StreamRequest,
	Response,
	StreamResponse,
	StreamCompleteResponse,
	Count
};

class FZenWebSocketMessage
{
	struct FHeader
	{
		static constexpr uint32_t ExpectedMagic = 0x7a776d68;

		uint64 MessageSize = 0;
		uint32 Magic = ExpectedMagic;
		uint32 CorrelationId = 0;
		uint32 StatusCode = 200;
		EZenWebSocketMessageType MessageType = EZenWebSocketMessageType::Invalid;
		uint8 Reserved[3] = {0};
	};

	static_assert(sizeof(FHeader) == 24); 

public:
	static constexpr uint64 HeaderSize = sizeof(FHeader);

	FZenWebSocketMessage() = default;

	EZenWebSocketMessageType MessageType() const { return MessageHeader.MessageType; }
	void SetMessageType(EZenWebSocketMessageType InType) { MessageHeader.MessageType = InType; }

	uint64 MessageSize() const { return MessageHeader.MessageSize; }
	uint64 TotalSize() const { return HeaderSize + MessageHeader.MessageSize; }

	uint32 CorrelationId() const { return MessageHeader.CorrelationId; }
	void SetCorrelationId(uint32 Id) { MessageHeader.CorrelationId = Id; }

	FCbPackage& Body() { return MessageBody; }
	FCbPackage&& ConsumeBody() { return MoveTemp(MessageBody); }
	void SetBody(FCbPackage&& Body) { MessageBody = MoveTemp(Body); }

	bool TryLoadHeader(FMemoryView HeaderView);
	void Save(FArchive& Ar);

	static uint32 GenerateCorrelationId()
	{
		return NextCorrelationId.IncrementExchange();
	}

private:
	static TAtomic<uint32> NextCorrelationId;

	FHeader		MessageHeader;
	FCbPackage	MessageBody;
};

bool FZenWebSocketMessage::TryLoadHeader(FMemoryView HeaderView)
{
	if (HeaderView.GetSize() < HeaderSize)
	{
		return false;
	}

	FMemory::Memcpy(&MessageHeader, HeaderView.GetData(), HeaderSize);

	const bool bIsValidHeader =
		MessageHeader.Magic == FHeader::ExpectedMagic &&
		MessageHeader.StatusCode > 0 &&
		static_cast<uint8>(MessageHeader.MessageType) > static_cast<uint8>(EZenWebSocketMessageType::Invalid) &&
		static_cast<uint8>(MessageHeader.MessageType) < static_cast<uint8>(EZenWebSocketMessageType::Count);

	return bIsValidHeader;
}

void FZenWebSocketMessage::Save(FArchive& Ar)
{
	Ar.Serialize(&MessageHeader, FZenWebSocketMessage::HeaderSize);
	MessageBody.Save(Ar);
	
	const int64 End = Ar.Tell();

	MessageHeader.MessageSize = End - int64(FZenWebSocketMessage::HeaderSize);
	MessageHeader.StatusCode = 200;

	Ar.Seek(0);
	Ar.Serialize(&MessageHeader, FZenWebSocketMessage::HeaderSize);
	Ar.Seek(End);
}

TAtomic<uint32> FZenWebSocketMessage::NextCorrelationId{1};

////////////////////////////////////////////////////////////////////////////////
enum class EParseMessageStatus : uint32
{
	Error,
	Continue,
	Done
};

class FZenMessageParser
{
public:
	FZenMessageParser() = default;

	EParseMessageStatus		ParseMessage(FMemoryView MessageData, uint64& OutBytesParsed, FString& OutReason);
	void					Reset();
	FZenWebSocketMessage&&	ConsumeMessage() { return MoveTemp(Message); }

private:
	FLargeMemoryWriter Ar;
	FZenWebSocketMessage Message;
};

void FZenMessageParser::Reset()
{
	Ar.Seek(0);
	Message = FZenWebSocketMessage();
}

EParseMessageStatus FZenMessageParser::ParseMessage(FMemoryView MessageData, uint64& OutBytesParsed, FString& OutReason)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FZenMessageParser::ParseMessage);

	const uint64 PrevPos = Ar.Tell();

	if (Ar.Tell() < FZenWebSocketMessage::HeaderSize)
	{
		FMemoryView HeaderView = MessageData.Left(FZenWebSocketMessage::HeaderSize - Ar.Tell());

		Ar.Serialize(const_cast<void*>(HeaderView.GetData()), HeaderView.GetSize());
		OutBytesParsed = Ar.Tell() - PrevPos;
		
		MessageData.RightChopInline(HeaderView.GetSize());

		if (Ar.Tell() < FZenWebSocketMessage::HeaderSize)
		{
			return EParseMessageStatus::Continue;
		}
		
		if (Message.TryLoadHeader(Ar.GetView()) == false)
		{
			OutReason = TEXT("Invalid websocket header");

			return EParseMessageStatus::Error;
		}

		if (Message.MessageSize() == 0)
		{
			return EParseMessageStatus::Done;
		}

		//Ar.Reserve(Message.MessageSize());
	}

	if (MessageData.IsEmpty() == false)
	{
		const uint64 RemainingMessageSize = Message.MessageSize() - (Ar.Tell() - FZenWebSocketMessage::HeaderSize);
		FMemoryView BodyView = MessageData.Left(RemainingMessageSize);

		Ar.Serialize(const_cast<void*>(BodyView.GetData()), BodyView.GetSize());
		OutBytesParsed = Ar.Tell() - PrevPos;
	}

	EParseMessageStatus Status = EParseMessageStatus::Continue;

	if (Ar.Tell() == FZenWebSocketMessage::HeaderSize + Message.MessageSize())
	{
		Status = EParseMessageStatus::Done;

		FMemoryView BodyView = Ar.GetView();
		BodyView.RightChopInline(FZenWebSocketMessage::HeaderSize);
		FMemoryReaderView BodyAr(BodyView);

		FCbPackage Body;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(CbPackage::TryLoad);

			if (Body.TryLoad(BodyAr))
			{
				Message.SetBody(MoveTemp(Body));
			}
			else
			{
				OutReason = TEXT("Invalid CbPackage");

				Status = EParseMessageStatus::Error;
			}
		}
	}

	return Status;
}

////////////////////////////////////////////////////////////////////////////////
class FZenClient final : public IZenClient
{
public:
	FZenClient();
	virtual ~FZenClient();

	virtual bool Connect(FStringView Host, int32 Port) override;
	virtual void Disconnect() override;
	virtual bool IsConnected() const override { return bConnected; }

	virtual bool SendRequest(FCbPackage&& Request, FOnStreamResponse&& OnResponse) override;
	virtual bool SendRequest(FCbObject&& Request, FOnStreamResponse&& OnResponse) override;
	virtual bool SendStreamRequest(FCbObject&& Request, FOnStreamResponse&& OnStreamResponse) override;

private:
	bool SendRequestMessage(FZenWebSocketMessage&& Request, FOnStreamResponse&& OnResponse);
	void ReadMessageThreadEntry();
	bool HandleMessage(FMemoryView MessageData, FString& OutReason);
	void DispatchMessage(FZenWebSocketMessage&& Message);
	void CloseConnection(FStringView Reason);

	struct FPendingRequest
	{
		FPendingRequest(FOnStreamResponse&& Callback)
			: OnStreamResponse(MoveTemp(Callback))
			, StartTime(FPlatformTime::Seconds()) {}

		FPendingRequest(const FPendingRequest&) = delete;
		FPendingRequest& operator=(const FPendingRequest&) = delete;
		
		FOnStreamResponse OnStreamResponse;
		FGraphEventRef LastCompletionEvent;
		const double StartTime;
	};

	struct FDispatchStreamResponseTask
	{
		FDispatchStreamResponseTask(TSharedPtr<FPendingRequest> InRequest, FCbPackage&& InResponse, bool bInComplete)
			: Request(InRequest), Response(MoveTemp(InResponse)), bComplete(bInComplete)
		{
			check(InRequest);
		}

		FORCEINLINE TStatId GetStatId() const { return TStatId(); }

		ENamedThreads::Type GetDesiredThread() { return ENamedThreads::AnyBackgroundHiPriTask; }

		static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }

		FORCEINLINE void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& CompletionGraphEvent)
		{
			Request->OnStreamResponse({ MoveTemp(Response), bComplete ? EZenStreamStatus::Completed :  EZenStreamStatus::Ok });
		}

		TSharedPtr<FPendingRequest> Request;
		FCbPackage Response;
		bool bComplete;
	};

	using FPendingRequests = TMap<uint32_t, TSharedPtr<FPendingRequest>>;

	TUniquePtr<FSocket>				Socket;
	TFuture<void>					ReadMessageThread;
	TSharedPtr<FInternetAddr>		HostAddr;
	TUniquePtr<FZenMessageParser>	Parser;
	FPendingRequests				PendingRequests;
	FCriticalSection				RequestsCriticalSection;
	TArray<uint8>					ReadBuffer;
	TAtomic<bool>					bConnected{false};
	TAtomic<bool>					bStopRequested{false};
};

FZenClient::FZenClient()
{
	const int32 ReadBufferSize = 256 << 10;

	Parser = MakeUnique<FZenMessageParser>();
	ReadBuffer.SetNum(ReadBufferSize);
}

FZenClient::~FZenClient()
{
	Disconnect();
}

bool FZenClient::Connect(FStringView Host, int32 Port)
{
	ISocketSubsystem& SocketSubsystem = *ISocketSubsystem::Get();

	TSharedPtr<FInternetAddr> Addr = SocketSubsystem.GetAddressFromString(FString(Host));
	Addr->SetPort(Port);

	Socket.Reset(SocketSubsystem.CreateSocket(NAME_Stream, TEXT("ZenClient"), Addr->GetProtocolType()));

	if (Socket.IsValid() == false)
	{
		UE_LOGF(LogZenClient, Error, "connect to server FAILED, reason 'failed to create socket'");

		return false;
	}
	
	int32 NewReadBufferSize = 0;
	if (Socket->SetReceiveBufferSize(ReadBuffer.Num(), NewReadBufferSize))
	{
		UE_LOGF(LogZenClient, Log, "read buffer size set to %dB", NewReadBufferSize);

		ReadBuffer.SetNum(NewReadBufferSize);
	}

	//Socket->SetLoopBackFastPath(true);
	Socket->SetNoDelay(true);

	if (Socket->Connect(*Addr) == false)
	{
		UE_LOGF(LogZenClient, Error, "connect to server FAILED");

		return false;
	}

	HostAddr = Addr;

	TArray<FString> Protocols;
	TAnsiStringBuilder<1024> UpgradeRequest;
	CreateHttpUpgradeRequest(Protocols, UpgradeRequest);

	if (SendData(*Socket, FMemoryView(UpgradeRequest.GetData(), UpgradeRequest.Len())) == false)
	{
		UE_LOGF(LogZenClient, Error, "connect to server FAILED, reason 'failed to send handshake request'");
		
		return false;
	}

	TAnsiStringBuilder<1024> UpgradeResponse;
	if (ReceieveHttpUpgradeResponse(*Socket, UpgradeResponse) == false)
	{
		UE_LOGF(LogZenClient, Error, "connect to server FAILED, reason 'failed to receive handshake request'");

		return false;
	}

	FString Reason;
	if (ValidateHandshake(UpgradeResponse.ToView(), Reason) == false)
	{
		UE_LOGF(LogZenClient, Error, "connect to server FAILED, reason '%ls'", *Reason);

		return false;
	}

	const uint32 StackSize = 0; 
	ReadMessageThread = AsyncThread([this] { return ReadMessageThreadEntry(); }, StackSize, TPri_AboveNormal);
	
	bConnected = true;

	return true;
}

void FZenClient::Disconnect()
{
	CloseConnection(TEXTVIEW(""));
}

bool FZenClient::SendRequestMessage(FZenWebSocketMessage&& RequestMessage, FOnStreamResponse&& OnResponse)
{
	FLargeMemoryWriter Ar;
	RequestMessage.Save(Ar);

	TSharedPtr<FPendingRequest> PendingRequest = MakeShared<FPendingRequest>(MoveTemp(OnResponse));

	{
		FScopeLock _(&RequestsCriticalSection);
		PendingRequests.Emplace(RequestMessage.CorrelationId(), MoveTemp(PendingRequest));
	}

	return SendData(*Socket, Ar.GetView());
}

bool FZenClient::SendRequest(FCbPackage&& Request, FOnStreamResponse&& OnResponse)
{
	FZenWebSocketMessage RequestMessage;

	RequestMessage.SetMessageType(EZenWebSocketMessageType::Request);
	RequestMessage.SetCorrelationId(FZenWebSocketMessage::GenerateCorrelationId());
	RequestMessage.SetBody(MoveTemp(Request));

	return SendRequestMessage(MoveTemp(RequestMessage), MoveTemp(OnResponse));
}

bool FZenClient::SendRequest(FCbObject&& Request, FOnStreamResponse&& OnResponse)
{
	FCbPackage Pkg;
	Pkg.SetObject(MoveTemp(Request));

	return SendRequest(MoveTemp(Pkg), MoveTemp(OnResponse));
}

bool FZenClient::SendStreamRequest(FCbObject&& Request, FOnStreamResponse&& OnStreamResponse)
{
	FZenWebSocketMessage StreamRequestMessage;

	StreamRequestMessage.SetMessageType(EZenWebSocketMessageType::StreamRequest);
	StreamRequestMessage.SetCorrelationId(FZenWebSocketMessage::GenerateCorrelationId());
	StreamRequestMessage.SetBody(FCbPackage(MoveTemp(Request)));

	return SendRequestMessage(MoveTemp(StreamRequestMessage), MoveTemp(OnStreamResponse));
}

void FZenClient::ReadMessageThreadEntry()
{
	FString Reason;

	while(bStopRequested == false)
	{
		int32 BytesRead = 0;
		{
			if (Socket->Recv(ReadBuffer.GetData(), ReadBuffer.Num(), BytesRead) == false)
			{
				Reason = TEXT("receive message FAILED");
				break;
			}
		}

		if (HandleMessage(FMemoryView(ReadBuffer.GetData(), BytesRead), Reason) == false)
		{
			break;
		}
	}

	if (bStopRequested == false)
	{
		CloseConnection(Reason);
	}
}

bool FZenClient::HandleMessage(FMemoryView MessageData, FString& OutReason)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ZenClient::HandleMessage);

	while (MessageData.IsEmpty() == false)
	{
		uint64 BytesParsed = 0;
		const EParseMessageStatus Status = Parser->ParseMessage(MessageData, BytesParsed, OutReason);

		MessageData.RightChopInline(BytesParsed);

		if (Status == EParseMessageStatus::Continue)
		{
			check(MessageData.IsEmpty());

			return true;
		}

		if (Status == EParseMessageStatus::Error)
		{
			return false;
		}

		DispatchMessage(Parser->ConsumeMessage());

		Parser->Reset();
	}

	return true;
}

void FZenClient::DispatchMessage(FZenWebSocketMessage&& Message)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ZenClient::DispatchMessage);

	const uint64 TotalMessageSize = Message.TotalSize();
	
	switch(Message.MessageType())
	{
	case EZenWebSocketMessageType::Notification:
	{
		check(false);
		break;
	}
	case EZenWebSocketMessageType::Response:
	case EZenWebSocketMessageType::StreamResponse:
	case EZenWebSocketMessageType::StreamCompleteResponse:
	{
		const bool bIsStreamComplete =
			Message.MessageType() == EZenWebSocketMessageType::StreamCompleteResponse || 
			Message.MessageType() == EZenWebSocketMessageType::Response;

		TGraphTask<FDispatchStreamResponseTask>* Task = nullptr;
		double StartTime = 0.0;

		{
			FScopeLock _(&RequestsCriticalSection);
			TSharedPtr<FPendingRequest> PendingRequest = PendingRequests.FindChecked(Message.CorrelationId());

			if (bIsStreamComplete)
			{
				PendingRequests.Remove(Message.CorrelationId());
				StartTime = PendingRequest->StartTime;
			}

			FGraphEventArray CompletionEvents;
			if (PendingRequest->LastCompletionEvent.IsValid())
			{
				CompletionEvents.Add(PendingRequest->LastCompletionEvent);
			}

			Task = TGraphTask<FDispatchStreamResponseTask>::CreateTask(&CompletionEvents).ConstructAndHold(PendingRequest, Message.ConsumeBody(), bIsStreamComplete);
			PendingRequest->LastCompletionEvent = Task->GetCompletionEvent();
		}

		Task->Unlock();

		if (bIsStreamComplete)
		{
			TRACE_COUNTER_ADD(ZenClient_StreamedBytes, TotalMessageSize);
			TRACE_COUNTER_ADD(ZenClient_StreamedSeconds, FPlatformTime::Seconds() - StartTime);
		}

		break;
	}
	default:
		UE_LOGF(LogZenClient, Warning, "dispatch message FAILED, reason 'invalid message type (%d)'", int(Message.MessageType()));
		break;
	}
}

void FZenClient::CloseConnection(FStringView Reason)
{
	if (bConnected.Exchange(false) == true)
	{
		UE_CLOGF(Reason.IsEmpty() == false, LogZenClient, Warning, "closing connection, reason '%ls'", *FString(Reason));

		check(PendingRequests.Num() == 0);

		bStopRequested = true;

		if (Socket)
		{
			Socket->Close();
		}
		
		ReadMessageThread.Wait();
		Socket.Reset();
	}
}

TUniquePtr<IZenClient> IZenClient::Create()
{
	return MakeUnique<FZenClient>();
}

////////////////////////////////////////////////////////////////////////////////
class FZenClientPool final : public IZenClientPool
{
public:
	FZenClientPool() = default;
	virtual ~FZenClientPool();

	virtual bool Connect(FStringView Host, int32 Port, int32 PoolSize = 8) override;
	virtual void Disconnect() override;

	virtual inline bool IsConnected() const override { return bConnected; }

	virtual bool SendRequest(FCbPackage&& Request, FOnStreamResponse&& OnResponse) override;
	virtual bool SendRequest(FCbObject&& Request, FOnStreamResponse&& OnResponse) override;
	virtual bool SendStreamRequest(FCbObject&& Request, FOnStreamResponse&& OnStreamResponse) override;

private:

#define UE_WITH_ZEN_NON_BLOCKING_POOL (1)

#if UE_WITH_ZEN_NON_BLOCKING_POOL
	struct FPoolEntry
	{
		std::atomic<uint8> bAllocated{0u};
		FZenClient Client;
	};
#else
	struct FPoolEntry
	{
		FZenClient Client;
		FPoolEntry* Next = nullptr;
	};
#endif

	using FPool = TArray<FPoolEntry>;

	FPoolEntry* GetClient();
	void ReleaseClient(FPoolEntry* Entry);

	FPool				Pool;
#if UE_WITH_ZEN_NON_BLOCKING_POOL
	TAtomic<int32>		PoolEntryStartIndex{0};
#else
	FPoolEntry*			PoolHead = nullptr;
	FPoolEntry*			PoolTail = nullptr;
#endif
	FCriticalSection	PoolCS;
	FEventRef			PoolAvailable;
	TAtomic<bool>		bConnected{false};
};

FZenClientPool::~FZenClientPool()
{
	Disconnect();
}

bool FZenClientPool::Connect(FStringView Host, int32 Port, int32 PoolSize)
{
	check(bConnected == false);

	Pool.SetNum(PoolSize);

	for (FPoolEntry& Entry : Pool)
	{
		if (Entry.Client.Connect(Host, Port) == false)
		{
			return false;		
		}
	}
	
	bConnected = true;

#if UE_WITH_ZEN_NON_BLOCKING_POOL
	for (FPoolEntry& Entry : Pool)
	{
		Entry.bAllocated = 0u;
	}
#else
	PoolHead = PoolTail = &Pool[0];

	for (int32 Index = 1; Index < PoolSize; ++Index)
	{
		Pool[Index].Next = PoolTail;
		PoolTail = &Pool[Index];
	}

#endif

	return bConnected;
}

void FZenClientPool::Disconnect()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ZenClientPool::Disconnect);

	if (bConnected.Exchange(false) == true)
	{
		for (FPoolEntry& Entry : Pool)
		{
			Entry.Client.Disconnect();
		}
		Pool.Empty();
	}
}

bool FZenClientPool::SendRequest(FCbPackage&& Request, FOnStreamResponse&& OnResponse)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ZenClientPool::SendRequest);

	bool bResult = false;

	if (FPoolEntry* Entry = GetClient())
	{	
		bResult = Entry->Client.SendRequest(MoveTemp(Request), MoveTemp(OnResponse));

		ReleaseClient(Entry);
	}

	return bResult;
}

bool FZenClientPool::SendRequest(FCbObject&& Request, FOnStreamResponse&& OnResponse)
{
	FCbPackage Pkg;
	Pkg.SetObject(MoveTemp(Request));

	return SendRequest(MoveTemp(Pkg), MoveTemp(OnResponse));
}

bool FZenClientPool::SendStreamRequest(FCbObject&& Request, FOnStreamResponse&& OnStreamResponse)
{
	bool bResult = false;

	if (FPoolEntry* Entry = GetClient())
	{
		bResult = Entry->Client.SendStreamRequest(MoveTemp(Request), MoveTemp(OnStreamResponse));
		
		ReleaseClient(Entry);
	}

	return bResult;
}

FZenClientPool::FPoolEntry* FZenClientPool::GetClient()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ZenClientPool::WaitForConnection);

#if UE_WITH_ZEN_NON_BLOCKING_POOL

	const int32 PoolSize = Pool.Num();
	int32 EntryIndex = PoolEntryStartIndex.IncrementExchange() % PoolSize;
	int32 SpinCount = 0;
	
	for (;;)
	{
		FPoolEntry& Entry = Pool[EntryIndex];
			
		if (Entry.bAllocated.load(std::memory_order_relaxed) == false)
		{
			uint8 Expected = 0u;
			if (Entry.bAllocated.compare_exchange_strong(Expected, 1u))
			{
				return &Entry;
			}
		}

		EntryIndex = (EntryIndex + 1) % PoolSize;
		
		if (++SpinCount >= PoolSize * 2)
		{
			SpinCount = 0;
			FPlatformProcess::Sleep(0.01f);
		}
	}
#else
	for(;;)
	{
		{
			FScopeLock _(&PoolCS);

			if (FPoolEntry* Entry = PoolHead)
			{
				if (Entry->Next)
				{
					PoolHead = Entry->Next;
				}
				else
				{
					PoolHead = PoolTail = nullptr;
				}

				Entry->Next = nullptr;
				return Entry;
			}
		}

		PoolAvailable->Wait();
	}
#endif
}

void FZenClientPool::ReleaseClient(FZenClientPool::FPoolEntry* Entry)
{
#if UE_WITH_ZEN_NON_BLOCKING_POOL
	uint8 Expected = 1u;
	Entry->bAllocated.compare_exchange_strong(Expected, 0u);
#else
	check(Entry->Next == nullptr);

	{
		FScopeLock _(&PoolCS);
		
		if (PoolTail)
		{
			check(PoolTail->Next == nullptr);
			PoolTail->Next = Entry;
		}
		else
		{
			PoolHead = Entry;
		}
		
		PoolTail = Entry;
	}
	
	PoolAvailable->Trigger();
#endif
}

TUniquePtr<IZenClientPool> IZenClientPool::Create()
{
	return MakeUnique<FZenClientPool>();
}

} // namespace UE::Zen
