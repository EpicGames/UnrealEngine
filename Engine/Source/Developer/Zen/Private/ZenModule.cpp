// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "ThirdParty/UECurl.h"

DEFINE_LOG_CATEGORY_STATIC(LogZenModule, Log, All);

class FZenModule
	: public IModuleInterface
{
public:
	// IModuleInterface interface
	virtual void StartupModule() override
	{
		FModuleManager::Get().LoadModuleChecked(TEXT("Sockets"));
		CURLcode InitResult = (CURLcode)UE::Curl::ConditionalInitialize(InitializeData);
		if (InitResult != CURLE_OK)
		{
			UE_LOGF(LogZenModule, Warning, "FZenModule could not initialize libcurl (result=%d).", (int32)InitResult);
		}
	}
	virtual void ShutdownModule() override
	{
		UE::Curl::ConditionalShutdown(InitializeData);
	}
	UE::Curl::FCurlInitializeData InitializeData;
};

IMPLEMENT_MODULE(FZenModule, Zen)
