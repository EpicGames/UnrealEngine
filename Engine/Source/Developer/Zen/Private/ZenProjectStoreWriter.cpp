// Copyright Epic Games, Inc. All Rights Reserved.

#include "Experimental/ZenProjectStoreWriter.h"

#include "Containers/StringView.h"
#include "Dom/JsonObject.h"
#include "Experimental/ZenServerInterface.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformMisc.h"
#include "Policies/PrettyJsonPrintPolicy.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "SocketSubsystem.h"

namespace UE::Zen
{

FZenProjectStoreWriter::FZenProjectStoreWriter(const TCHAR* TargetFileName)
: OplogMarker(IFileManager::Get().CreateFileWriter(TargetFileName))
{
}

bool FZenProjectStoreWriter::Write(FStringView ServiceHostName, uint16 ServicePort, const FString& HostAuthJson, bool bIsRunningLocally, FStringView ProjectId, FStringView OplogId, FName TargetPlatformName)
{
	if (!OplogMarker)
	{
		return false;
	}

	TSharedRef<TJsonWriter<UTF8CHAR, TPrettyJsonPrintPolicy<UTF8CHAR>>> Writer = TJsonWriterFactory<UTF8CHAR, TPrettyJsonPrintPolicy<UTF8CHAR>>::Create(OplogMarker.Get());
	Writer->WriteObjectStart();
	Writer->WriteObjectStart(TEXT("zenserver"));
	Writer->WriteValue(TEXT("islocalhost"), bIsRunningLocally);
	Writer->WriteValue(TEXT("hostname"), ServiceHostName);
	if (bIsRunningLocally)
	{
		ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get();
		if (SocketSubsystem != nullptr)
		{
			TArray<TSharedPtr<FInternetAddr>> Addresses;
			if (SocketSubsystem->GetLocalAdapterAddresses(Addresses))
			{
				Writer->WriteArrayStart("remotehostnames");
				for (const TSharedPtr<FInternetAddr>& Address : Addresses)
				{
					Writer->WriteValue(Address->ToString(false));
				}
				FString MachineHostName;
				if (SocketSubsystem->GetHostName(MachineHostName))
				{
					// Try to acquire FQDN
					FAddressInfoResult GAIRequest = SocketSubsystem->GetAddressInfo(*MachineHostName, nullptr, EAddressInfoFlags::AllResultsWithMapping | EAddressInfoFlags::OnlyUsableAddresses | EAddressInfoFlags::AllResults | EAddressInfoFlags::FQDomainName, NAME_None);
					if (GAIRequest.ReturnCode == SE_NO_ERROR && !GAIRequest.CanonicalNameResult.IsEmpty())
					{
						MachineHostName = GAIRequest.CanonicalNameResult;
					}

					Writer->WriteValue(TEXT("hostname://") + MachineHostName);
				}
#if PLATFORM_MAC
				// Store the Bonjour hostname when on a mac so that the client can attempt to connect via USB.
				// This is done since the "Link Local" IP of the USB connection changes everytime
				// the cable is plugged in, so there is no stable IP to save.
				bool bAppleTarget = TargetPlatformName == TEXT("IOS") || TargetPlatformName == TEXT("TVOS");
				if (bAppleTarget)
				{
					FString MacBonjourName = FPlatformMisc::GetBonjourName();
					
					// If there is no Bonjour name, fallback to the mac's hostname
					if (MacBonjourName.IsEmpty())
					{
						MacBonjourName = FPlatformProcess::ComputerName();
					}
					
					if (!MacBonjourName.IsEmpty())
					{
						Writer->WriteValue(TEXT("macserver://") + MacBonjourName);
					}
				}
#endif
				Writer->WriteArrayEnd();
			}
		}
	}
	Writer->WriteValue(TEXT("hostport"), ServicePort);
	if (!HostAuthJson.IsEmpty())
	{
		TSharedPtr<FJsonObject> AuthObject;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(HostAuthJson);
		if (FJsonSerializer::Deserialize(Reader, AuthObject) && AuthObject.IsValid())
		{
			Writer->WriteObjectStart(TEXT("hostauth"));
			for (const auto& Pair : AuthObject->Values)
			{
				FString Key(Pair.Key);
				if (Pair.Value->Type == EJson::String)
				{
					Writer->WriteValue(Key, Pair.Value->AsString());
				}
				else if (Pair.Value->Type == EJson::Boolean)
				{
					Writer->WriteValue(Key, Pair.Value->AsBool());
				}
				else if (Pair.Value->Type == EJson::Number)
				{
					Writer->WriteValue(Key, Pair.Value->AsNumber());
				}
			}
			Writer->WriteObjectEnd();
		}
	}
	Writer->WriteValue(TEXT("projectid"), ProjectId);
	Writer->WriteValue(TEXT("oplogid"), OplogId);
	Writer->WriteObjectEnd();
	Writer->WriteObjectEnd();
	Writer->Close();

	OplogMarker.Reset();
	return true;
}

}
