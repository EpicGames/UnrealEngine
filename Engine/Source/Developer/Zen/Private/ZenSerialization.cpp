// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZenSerialization.h"

#include "Compression/OodleDataCompression.h"
#include "HAL/FileManager.h"
#include "Misc/ScopeExit.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinaryPackage.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryValidation.h"
#include "Serialization/CompactBinaryWriter.h"

#if PLATFORM_WINDOWS
#	include "Windows/WindowsHWrapper.h"
#endif // PLATFORM_WINDOWS

DEFINE_LOG_CATEGORY_STATIC(LogZenSerialization, Log, All);

namespace UE::Zen::OpLog
{

void SaveCbAttachment(const FCbAttachment& Attachment, FCbWriter& Writer)
{
	if (Attachment.IsCompressedBinary())
	{
		Writer.AddBinary(Attachment.AsCompressedBinary().GetCompressed());
		Writer.AddBinaryAttachment(Attachment.GetHash());
	}
	else if (Attachment.IsNull())
	{
		Writer.AddBinary(FMemoryView());
	}
	else
	{
		// NOTE: All attachments needs to be compressed
		checkNoEntry();
	}
}

void SaveCbPackage(const FCbPackage& Package, FCbWriter& Writer)
{
	if (const FCbObject& RootObject = Package.GetObject())
	{
		Writer.AddObject(RootObject);
		Writer.AddObjectAttachment(Package.GetObjectHash());
	}
	for (const FCbAttachment& Attachment : Package.GetAttachments())
	{
		SaveCbAttachment(Attachment, Writer);
	}
	Writer.AddNull();
}

void SaveCbPackage(const FCbPackage& Package, FArchive& Ar)
{
	FCbWriter Writer;
	SaveCbPackage(Package, Writer);
	Writer.Save(Ar);
}

bool TryLoadCbPackage(FCbPackage& Package, FArchive& Ar, FCbBufferAllocator Allocator)
{
	uint8 StackBuffer[64];
	const auto StackAllocator = [&Allocator, &StackBuffer](uint64 Size) -> FUniqueBuffer
	{
		return Size <= sizeof(StackBuffer) ? FUniqueBuffer::MakeView(StackBuffer, Size) : Allocator(Size);
	};

	Package = FCbPackage();
	for (;;)
	{
		FCbField ValueField = LoadCompactBinary(Ar, StackAllocator);
		if (!ValueField)
		{
			UE_LOGF(LogZenSerialization, Warning, "Package is malformed, can't read compact binary data");
			Ar.SetError();
			return false;
		}
		if (ValueField.IsNull())
		{
			return true;
		}
		else if (ValueField.IsBinary())
		{
			const FMemoryView View = ValueField.AsBinaryView();
			if (View.GetSize() > 0)
			{
				FSharedBuffer Buffer = FSharedBuffer::MakeView(View, ValueField.GetOuterBuffer()).MakeOwned();
				FCbField HashField = LoadCompactBinary(Ar, StackAllocator);
				const FIoHash& Hash = HashField.AsAttachment();
				if (HashField.HasError() || FIoHash::HashBuffer(Buffer) != Hash)
				{
					UE_LOGF(LogZenSerialization, Warning, "Package attachment has malformed/invalid attachment hash field");
					Ar.SetError();
					return false;
				}
				if (HashField.IsObjectAttachment())
				{
					Package.AddAttachment(FCbAttachment(FCbObject(MoveTemp(Buffer)), Hash));
				}
				else
				{
					Package.AddAttachment(FCbAttachment(FCompositeBuffer(MoveTemp(Buffer)), Hash));
				}
			}
		}
		else
		{
			FCbObject Object = ValueField.AsObject();
			if (ValueField.HasError())
			{
				UE_LOGF(LogZenSerialization, Warning, "Package attachment has malformed/invalid object field");
				Ar.SetError();
				return false;
			}

			if (Object)
			{
				FCbField HashField = LoadCompactBinary(Ar, StackAllocator);
				FIoHash ObjectHash = HashField.AsObjectAttachment();
				if (HashField.HasError() || Object.GetHash() != ObjectHash)
				{
					UE_LOGF(LogZenSerialization, Warning, "Package attachment has malformed/invalid object hash field");
					Ar.SetError();
					return false;
				}
				Package.SetObject(Object, ObjectHash);
			}
		}
	}
}

} // UE::Zen::OpLog

namespace UE::Zen::Http
{

struct CbPackageHeader
{
	uint32	HeaderMagic;
	uint32	AttachmentCount;
	uint32	Reserved1;
	uint32	Reserved2;
};

struct CbAttachmentEntry
{
	uint64	AttachmentSize;
	uint32	Flags;
	FIoHash	AttachmentHash;

	enum
	{
		IsCompressed = (1u << 0),	// Is marshaled using compressed buffer storage format
		IsObject = (1u << 1),		// Is compact binary object
		kIsError = (1u << 2),	    // Is error (compact binary formatted) object
		kIsLocalRef = (1u << 3),	// Is "local reference"
	};

};

struct CbAttachmentReferenceHeader
{
	uint64 PayloadByteOffset = 0;
	uint64 PayloadByteSize = ~0u;
	uint16 AbsolutePathLength = 0;

	// This header will be followed by UTF8 encoded absolute path to backing file
};

void SaveCbPackage(const FCbPackage& Package, FArchive& Ar)
{
	TConstArrayView<FCbAttachment> Attachments = Package.GetAttachments();
	const FCbObject& Object = Package.GetObject();
	FCompressedBuffer ObjectBuffer = FCompressedBuffer::Compress(Object.GetBuffer(), FOodleDataCompression::ECompressor::NotSet, FOodleDataCompression::ECompressionLevel::None);

	CbPackageHeader Hdr;
	Hdr.HeaderMagic = kCbPkgMagic;
	Hdr.AttachmentCount = Attachments.Num();
	Hdr.Reserved1 = 0;
	Hdr.Reserved2 = 0;

	Ar.Serialize(&Hdr, sizeof Hdr);

	// Root object metadata

	{
		CbAttachmentEntry Entry;
		Entry.AttachmentHash = ObjectBuffer.GetRawHash();
		Entry.AttachmentSize = ObjectBuffer.GetCompressedSize();
		Entry.Flags = CbAttachmentEntry::IsObject | CbAttachmentEntry::IsCompressed;

		Ar.Serialize(&Entry, sizeof Entry);
	}

	// Attachment metadata

	for (const FCbAttachment& Attachment : Attachments)
	{
		CbAttachmentEntry Entry;
		Entry.AttachmentHash = Attachment.GetHash();
		Entry.Flags = 0;

		if (Attachment.IsCompressedBinary())
		{
			Entry.AttachmentSize = Attachment.AsCompressedBinary().GetCompressedSize();
			Entry.Flags |= CbAttachmentEntry::IsCompressed;
		}
		else if (Attachment.IsBinary())
		{
			Entry.AttachmentSize = Attachment.AsCompositeBinary().GetSize();
		}
		else if (Attachment.IsNull())
		{
			checkNoEntry();
		}
		else if (Attachment.IsObject())
		{
			Entry.AttachmentSize = Attachment.AsObject().GetSize();
			Entry.Flags |= CbAttachmentEntry::IsObject;
		}
		else
		{
			checkNoEntry();
		}

		Ar.Serialize(&Entry, sizeof Entry);
	}

	// Root object

	Ar << ObjectBuffer;

	// Payloads back-to-back

	for (const FCbAttachment& Attachment : Attachments)
	{
		if (Attachment.IsCompressedBinary())
		{
			FCompressedBuffer Payload = Attachment.AsCompressedBinary();
			Ar << Payload;
		}
		else if (Attachment.IsBinary())
		{
			const FCompositeBuffer& Buffer = Attachment.AsCompositeBinary();
			FSharedBuffer SharedBuffer = Buffer.ToShared();

			Ar.Serialize((void*)SharedBuffer.GetData(), SharedBuffer.GetSize());
		}
		else if (Attachment.IsNull())
		{
			checkNoEntry();
		}
		else if (Attachment.IsObject())
		{
			checkNoEntry();
		}
		else
		{
			checkNoEntry();
		}
	}
}

bool TryLoadCbPackage(FCbPackage& Package, FArchive& Ar, FCbBufferAllocator Allocator)
{
	CbPackageHeader Hdr;
	Ar.Serialize(&Hdr, sizeof Hdr);
	if (Ar.IsError())
	{
		return false;
	}

	if (Hdr.HeaderMagic != kCbPkgMagic)
	{
		return false;
	}

	TArray<CbAttachmentEntry> AttachmentEntries;
	AttachmentEntries.SetNum(Hdr.AttachmentCount + 1);

	Ar.Serialize(AttachmentEntries.GetData(), (Hdr.AttachmentCount + 1) * sizeof(CbAttachmentEntry));
	if (Ar.IsError())
	{
		UE_LOG(LogZenSerialization, Warning, TEXT("Package invalid, unable to read %" UINT64_FMT " bytes for attachment header entries"), (Hdr.AttachmentCount + 1) * sizeof(CbAttachmentEntry));
		return false;
	}

	int Index = 0;

	for (const CbAttachmentEntry& Entry : AttachmentEntries)
	{
		FUniqueBuffer AttachmentData = FUniqueBuffer::Alloc(Entry.AttachmentSize);
		Ar.Serialize(AttachmentData.GetData(), AttachmentData.GetSize());
		if (Ar.IsError())
		{
			UE_LOG(LogZenSerialization, Warning, TEXT("Package invalid, unable to read %" UINT64_FMT " bytes of attachment data"), Entry.AttachmentSize);
			return false;
		}

		if (Entry.Flags & CbAttachmentEntry::kIsLocalRef)
		{
			// Marshal local reference - a "pointer" to the chunk backing file

			if (AttachmentData.GetSize() < sizeof(CbAttachmentReferenceHeader))
			{
				// We did not get the full package data - treat it as a malformed package
				UE_LOG(LogZenSerialization, Warning, TEXT("Package payload is not large enough while trying to read local ref header, need %" UINT64_FMT " bytes, but only %" UINT64_FMT " available."), sizeof(CbAttachmentReferenceHeader), AttachmentData.GetSize());
				Ar.SetError();
				return false;
			}

			if (!(Entry.Flags & CbAttachmentEntry::IsCompressed))
			{
				// We only support compressed attachments - treat it as an error
				UE_LOGF(LogZenSerialization, Error, "Attachment data is not compressed");
				Ar.SetError();
				return false;
			}

			const CbAttachmentReferenceHeader* AttachRefHdr = reinterpret_cast<CbAttachmentReferenceHeader*>(AttachmentData.GetData());
			const UTF8CHAR* PathPointer = reinterpret_cast<const UTF8CHAR*>(AttachRefHdr + 1);

			const uint64 ExpectedHeaderSize = sizeof(CbAttachmentReferenceHeader) + AttachRefHdr->AbsolutePathLength;
			if (AttachmentData.GetSize() < ExpectedHeaderSize)
			{
				// We did not get the full package data - treat it as a malformed package
				UE_LOG(LogZenSerialization, Warning, TEXT("Package payload is not large enough while trying to read local ref path, need %" UINT64_FMT " bytes, but only %" UINT64_FMT " available."), ExpectedHeaderSize, AttachmentData.GetSize());
				Ar.SetError();
				return false;
			}

			FCompressedBuffer CompBuf;
			FString Path(FUtf8StringView(PathPointer, static_cast<uint32_t>(AttachRefHdr->AbsolutePathLength)));

			const FStringView HandlePrefix(TEXT(":?#:"));
			if (Path.StartsWith(HandlePrefix))
			{
#if PLATFORM_WINDOWS
				FString		HandleString(Path.RightChop(HandlePrefix.Len()));

				uint64 HandleNum = 0;
				LexFromString(HandleNum, *HandleString);
				if (HandleNum != 0)
				{
					HANDLE Handle = reinterpret_cast<HANDLE>(HandleNum);
					ON_SCOPE_EXIT{ CloseHandle(Handle); };
					LARGE_INTEGER FileSize;
					BOOL OK = GetFileSizeEx(Handle, &FileSize);
					if (!OK)
					{
						UE_LOG(LogZenSerialization, Warning, TEXT("Unable to read local file via handle '%" UINT64_FMT "', treating it as a missing attachment"), HandleNum);
						++Index;
						continue;
					}
					FUniqueBuffer MemBuffer = Allocator(FileSize.QuadPart);

					LONGLONG Offset = 0;
					while (Offset < FileSize.QuadPart)
					{
						DWORD Length = static_cast<DWORD>(FMath::Min<LONGLONG>(0xffff'ffffu, FileSize.QuadPart - Offset));
						DWORD dwNumberOfBytesRead = 0;
						OVERLAPPED Ovl{};
						Ovl.Offset = DWORD(Offset & 0xffff'ffffu);
						Ovl.OffsetHigh = DWORD(Offset >> 32);
						BOOL  Success = ::ReadFile(Handle, (void*)MemBuffer.GetView().Mid(Offset, Length).GetData(), Length, &dwNumberOfBytesRead, &Ovl);
						if (!Success)
						{
							DWORD LastError = GetLastError();
							UE_LOG(LogZenSerialization, Warning, TEXT("Unable to read local file via handle '%" UINT64_FMT "', error: %u, treating it as a missing attachment"), HandleNum, LastError);
							break;
						}
						Offset += Length;
					}
					if (Offset != FileSize.QuadPart)
					{
						++Index;
						continue;
					}
					CompBuf = FCompressedBuffer::FromCompressed(MemBuffer.MoveToShared());
				}
#else // PLATFORM_WINDOWS
				// Only suported on Windows. For Linux we could potentially use pidfd_getfd() but that requires a fairly new Linux kernel/includes and
				// to deal with acceess rights etc.
				checkf(false, TEXT("Passing file handles is not supported on this platform"));
#endif // PLATFORM_WINDOWS
			}
			else
			{
				TUniquePtr<FArchive> ChunkReader(IFileManager::Get().CreateFileReader(*Path, FILEREAD_Silent));
				if (!ChunkReader)
				{
					// File does not exist on disk - treat it as missing
					UE_LOGF(LogZenSerialization, Warning, "Unable to read local file '%ls', treating it as a missing attachment", *Path);
					++Index;
					continue;
				}
				CompBuf = FCompressedBuffer::Load(*ChunkReader);
			}

			if (!CompBuf)
			{
				// File has wrong format - treat it as an error
				UE_LOGF(LogZenSerialization, Error, "Local file '%ls' format is not compressed", *Path);
				Ar.SetError();
				return false;
			}
			FCbAttachment Attachment(MoveTemp(CompBuf), Entry.AttachmentHash);
			Package.AddAttachment(Attachment);
		}
		else if (Entry.Flags & CbAttachmentEntry::IsCompressed)
		{
			FCompressedBuffer CompBuf(FCompressedBuffer::FromCompressed(AttachmentData.MoveToShared()));

			if (Entry.Flags & CbAttachmentEntry::IsObject)
			{
				checkf(Index == 0, TEXT("Object attachments are not currently supported"));

				Package.SetObject(FCbObject(CompBuf.Decompress()));
			}
			else
			{
				FCbAttachment Attachment(MoveTemp(CompBuf), Entry.AttachmentHash);
				Package.AddAttachment(Attachment);
			}
		}
		else /* not compressed */
		{
			if (Entry.Flags & CbAttachmentEntry::IsObject)
			{
				checkf(Index == 0, TEXT("Object attachments are not currently supported"));

				Package.SetObject(FCbObject(AttachmentData.MoveToShared()));
			}
			else
			{
				FCbAttachment Attachment(AttachmentData.MoveToShared(), Entry.AttachmentHash);
				Package.AddAttachment(Attachment);
			}
		}

		++Index;
	}
	return true;
}

} // UE::Zen::Http
