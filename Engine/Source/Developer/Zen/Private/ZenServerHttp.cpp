// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZenServerHttp.h"

#include "ZenBackendUtils.h"
#include "ZenSerialization.h"

#if PLATFORM_MICROSOFT
#	include "Microsoft/AllowMicrosoftPlatformTypes.h"
#endif

#if PLATFORM_WINDOWS
#	include <mstcpip.h>
#endif

#if !defined(PLATFORM_CURL_INCLUDE)
#include "curl/curl.h"
#endif

#if PLATFORM_MICROSOFT
#	include "Microsoft/HideMicrosoftPlatformTypes.h"
#endif

#include "Logging/LogMacros.h"
#include "Compression/CompressedBuffer.h"
#include "Containers/StringFwd.h"
#include "HAL/LowLevelMemTracker.h"
#include "Memory/CompositeBuffer.h"
#include "Misc/App.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinaryPackage.h"
#include "Serialization/CompactBinaryValidation.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/LargeMemoryReader.h"
#include "Serialization/LargeMemoryWriter.h"
#include "HAL/PlatformProcess.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "ZenSerialization.h"

LLM_DEFINE_TAG(ZenDDC, NAME_None, TEXT("DDCBackend"));

DEFINE_LOG_CATEGORY_STATIC(LogZenHttp, Log, All);

namespace UE::Zen {

#define UE_ZENDDC_BACKEND_WAIT_INTERVAL			0.01f
#define UE_ZENDDC_HTTP_DEBUG					0

	struct FZenHttpRequest::FStatics
	{
		static size_t StaticDebugCallback(CURL* Handle, curl_infotype DebugInfoType, char* DebugInfo, size_t DebugInfoSize, void* UserData);
		static size_t StaticReadFn(void* Ptr, size_t SizeInBlocks, size_t BlockSizeInBytes, void* UserData);
		static size_t StaticWriteHeaderFn(void* Ptr, size_t SizeInBlocks, size_t BlockSizeInBytes, void* UserData);
		static size_t StaticWriteBodyFn(void* Ptr, size_t SizeInBlocks, size_t BlockSizeInBytes, void* UserData);
		static size_t StaticSeekFn(void* UserData, curl_off_t Offset, int Origin);
#if PLATFORM_WINDOWS
		static int StaticSockoptFn(void* UserData, curl_socket_t CurlFd, curlsocktype Purpose);
#endif //PLATFORM_WINDOWS
	};

	FZenHttpRequest::FZenHttpRequest(const FZenServiceEndpoint& InEndpoint, bool bInLogErrors, uint32 InConnectTimeoutMS, uint32 InTimeoutMS)
		: bLogErrors(bInLogErrors)
		, ConnectTimeoutMS(InConnectTimeoutMS)
		, TimeoutMS(InTimeoutMS)
		, Domain(InEndpoint.GetURL())
		, Endpoint(InEndpoint)
	{
		Curl = curl_easy_init();
		Reset();
	}

	FZenHttpRequest::~FZenHttpRequest()
	{
		curl_easy_cleanup(Curl);
	}

	void FZenHttpRequest::Reset()
	{
		Headers.Reset();
		ResponseHeader.Reset();
		ResponseBuffer.Reset();
		ResponseCode = 0;
		bResponseFormatValid = true;
		ReadDataView = nullptr;
		WriteDataBufferPtr = nullptr;
		WriteHeaderBufferPtr = nullptr;
		BytesSent = 0;
		BytesReceived = 0;
		CurlResult = CURL_LAST;

		curl_easy_reset(Curl);

		// Options that are always set for all connections.
		curl_easy_setopt(Curl, CURLOPT_CONNECTTIMEOUT_MS, ConnectTimeoutMS > 0 ? ConnectTimeoutMS : 5000L);
		if (TimeoutMS > 0)
		{
			curl_easy_setopt(Curl, CURLOPT_TIMEOUT_MS, TimeoutMS);
		}
		curl_easy_setopt(Curl, CURLOPT_EXPECT_100_TIMEOUT_MS, 0);
		curl_easy_setopt(Curl, CURLOPT_NOSIGNAL, 1L);
		curl_easy_setopt(Curl, CURLOPT_DNS_CACHE_TIMEOUT, -1L); // Don't re-resolve names mid-session
		curl_easy_setopt(Curl, CURLOPT_BUFFERSIZE, 256 * 1024L);
		curl_easy_setopt(Curl, CURLOPT_NOPROXY, "*");
		curl_easy_setopt(Curl, CURLOPT_UPLOAD_BUFFERSIZE, 2 * 1024L * 1024L); // Max buffer size is 2MB. 
		// Response functions
		curl_easy_setopt(Curl, CURLOPT_HEADERDATA, this);
		curl_easy_setopt(Curl, CURLOPT_HEADERFUNCTION, &FZenHttpRequest::FStatics::StaticWriteHeaderFn);
		curl_easy_setopt(Curl, CURLOPT_WRITEDATA, this);
		curl_easy_setopt(Curl, CURLOPT_WRITEFUNCTION, &FZenHttpRequest::FStatics::StaticWriteBodyFn);
		// Rewind method, handle special error case where request need to rewind data stream
		curl_easy_setopt(Curl, CURLOPT_SEEKDATA, this);
		curl_easy_setopt(Curl, CURLOPT_SEEKFUNCTION, &FZenHttpRequest::FStatics::StaticSeekFn);
#if PLATFORM_WINDOWS
		curl_easy_setopt(Curl, CURLOPT_SOCKOPTFUNCTION, &FZenHttpRequest::FStatics::StaticSockoptFn);
#endif //PLATFORM_WINDOWS
		// Debug hooks
#if UE_ZENDDC_HTTP_DEBUG
		curl_easy_setopt(Curl, CURLOPT_DEBUGDATA, this);
		curl_easy_setopt(Curl, CURLOPT_DEBUGFUNCTION, &FZenHttpRequest::FStatics::StaticDebugCallback);
		curl_easy_setopt(Curl, CURLOPT_VERBOSE, 1L);
#endif

		if (Endpoint.GetSocketType() == FZenServiceEndpoint::ESocketType::Unix)
		{
			TAnsiStringBuilder<256> AsUtf8;
			AsUtf8 << Endpoint.GetName();
			curl_easy_setopt(Curl, CURLOPT_UNIX_SOCKET_PATH, AsUtf8.ToString());
		}
	}

	void FZenHttpRequest::Initialize(bool bInLogErrors)
	{
		bLogErrors = bInLogErrors;
	}

	FAnsiStringView FZenHttpRequest::GetError() const
	{
		if (CurlResult != CURLE_OK)
		{
			return curl_easy_strerror((CURLcode)CurlResult);
		}
		return {};
	}

	void FZenHttpRequest::AddHeader(FStringView Header, FStringView Value)
	{
		TStringBuilder<128> Sb;
		Sb << Header << TEXTVIEW(": ") << Value;
		Headers.Emplace(Sb);
	}

	FZenHttpRequest::Result FZenHttpRequest::PerformBlockingPut(const TCHAR* Uri, const FCompositeBuffer& Buffer, EContentType ContentType)
	{
		uint64 ContentLength = 0u;

		ContentLength = Buffer.GetSize();
		curl_easy_setopt(Curl, CURLOPT_UPLOAD, 1L);
		curl_easy_setopt(Curl, CURLOPT_INFILESIZE, ContentLength);
		curl_easy_setopt(Curl, CURLOPT_READDATA, this);
		curl_easy_setopt(Curl, CURLOPT_READFUNCTION, &FZenHttpRequest::FStatics::StaticReadFn);

		AddHeader(TEXTVIEW("Content-Type"), GetMimeType(ContentType));

		ReadDataView = &Buffer;

		return PerformBlocking(Uri, RequestVerb::Put, ContentLength);
	}

	FZenHttpRequest::Result FZenHttpRequest::PerformBlockingPost(FStringView Uri, FCbObjectView Obj,
		EContentType AcceptType)
	{
		FLargeMemoryWriter Out;
		Obj.CopyTo(Out);

		return PerformBlockingPost(Uri, Out.GetView(), EContentType::CbObject, AcceptType);
	}

	FZenHttpRequest::Result FZenHttpRequest::PerformBlockingPostPackage(FStringView Uri, const FCbPackage& Package, EContentType AcceptType)
	{
		FLargeMemoryWriter Out;
		Http::SaveCbPackage(Package, Out);

		return PerformBlockingPost(Uri, Out.GetView(), EContentType::CbPackage, AcceptType);
	}

	FZenHttpRequest::Result FZenHttpRequest::PerformRpc(FStringView Uri, FCbObjectView Request, FCbPackage &OutResponse)
	{
		return ParseRpcResponse(
			PerformBlockingPost(Uri, Request, EContentType::CbPackage), OutResponse);
	}
		
	FZenHttpRequest::Result FZenHttpRequest::PerformRpc(FStringView Uri, const FCbPackage& Request, FCbPackage& OutResponse)
	{
		return ParseRpcResponse(
			PerformBlockingPostPackage(Uri, Request, EContentType::CbPackage), OutResponse);
	}

	FZenHttpRequest::Result FZenHttpRequest::ParseRpcResponse(FZenHttpRequest::Result ResultFromPost, FCbPackage& OutResponse)
	{
		if (ResultFromPost != Result::Success || !IsSuccessCode(ResponseCode))
		{
			return Result::Failed;
		}

		if (ResponseBuffer.Num())
		{
			{
				FLargeMemoryReader Ar(ResponseBuffer.GetData(), ResponseBuffer.Num());
				if (Http::TryLoadCbPackage(OutResponse, Ar))
				{
					return Result::Success;
				}
			}
			FLargeMemoryReader Ar(ResponseBuffer.GetData(), ResponseBuffer.Num());
			if (!OutResponse.TryLoad(Ar))
			{
				return Result::Failed;
			}
		}

		return Result::Success;
	}

	FCbPackage FZenHttpRequest::GetResponseAsPackage() const
	{
		const TArray64<uint8>& Response = GetResponseBuffer();
		FLargeMemoryReader Reader(Response.GetData(), Response.Num());

		FCbPackage Package;
		if (!Http::TryLoadCbPackage(Package, Reader))
		{
			return {};
		}

		return Package;
	}

	FZenHttpRequest::Result FZenHttpRequest::PerformBlockingPost(FStringView Uri, FMemoryView Payload,
		EContentType ContentType, EContentType AcceptType)
	{
		uint64 ContentLength = 0u;

		curl_easy_setopt(Curl, CURLOPT_POST, 1L);
		curl_easy_setopt(Curl, CURLOPT_POSTFIELDSIZE, Payload.GetSize());
		curl_easy_setopt(Curl, CURLOPT_READDATA, this);
		curl_easy_setopt(Curl, CURLOPT_READFUNCTION, &FZenHttpRequest::FStatics::StaticReadFn);

		AddHeader(TEXTVIEW("Content-Type"), GetMimeType(ContentType));
		if (AcceptType != EContentType::UnknownContentType)
		{
			AddHeader(TEXTVIEW("Accept"), GetMimeType(EContentType::CbPackage));
		}

		ContentLength = Payload.GetSize();

		FCompositeBuffer Buffer(FSharedBuffer::MakeView(Payload));
		ReadDataView = &Buffer;

		return PerformBlocking(Uri, RequestVerb::Post, ContentLength);
	}

	FZenHttpRequest::Result FZenHttpRequest::PerformBlockingDownload(FStringView Uri, TArray64<uint8>* Buffer, EContentType AcceptType)
	{
		curl_easy_setopt(Curl, CURLOPT_HTTPGET, 1L);
		WriteDataBufferPtr = Buffer;
		
		AddHeader(TEXTVIEW("Accept"), GetMimeType(AcceptType));

		return PerformBlocking(Uri, RequestVerb::Get, 0u);
	}

	FZenHttpRequest::Result FZenHttpRequest::PerformBlockingDownload(const TCHAR* Uri, FCbPackage& OutPackage)
	{
		curl_easy_setopt(Curl, CURLOPT_HTTPGET, 1L);
		OutPackage.Reset();

		AddHeader(TEXTVIEW("Accept"), GetMimeType(EContentType::CbPackage));

		// TODO: When PackageBytes can be written in segments directly, set the WritePtr to the OutPackage and use that
		TArray64<uint8> PackageBytes;
		WriteDataBufferPtr = &PackageBytes;
		Result LocalResult = PerformBlocking(Uri, RequestVerb::Get, 0u);
		if (IsSuccessCode(ResponseCode))
		{
			FLargeMemoryReader Ar(PackageBytes.GetData(), PackageBytes.Num());
			bResponseFormatValid = OutPackage.TryLoad(Ar);
		}
		return LocalResult;
	}

	FZenHttpRequest::Result FZenHttpRequest::PerformBlockingHead(FStringView Uri, EContentType AcceptType)
	{
		curl_easy_setopt(Curl, CURLOPT_NOBODY, 1L);
		
		AddHeader(TEXTVIEW("Accept"), GetMimeType(AcceptType));

		return PerformBlocking(Uri, RequestVerb::Head, 0u);
	}

	FZenHttpRequest::Result FZenHttpRequest::PerformBlockingDelete(const FStringView Uri)
	{
		curl_easy_setopt(Curl, CURLOPT_CUSTOMREQUEST, "DELETE");

		return PerformBlocking(Uri, RequestVerb::Delete, 0u);
	}

	static const char* GetSessionIdHeader() {
		static FCbObjectId SessionId = FApp::GetSessionObjectId();
		static const char* HeaderString = [&] {
			static TAnsiStringBuilder<64> SessionIdHeader;
			SessionIdHeader << "UE-Session: " << SessionId;
			return SessionIdHeader.GetData(); 
		}();

		return HeaderString;
	}

	static std::atomic<int> RequestId{1};

	FZenHttpRequest::Result FZenHttpRequest::PerformBlocking(FStringView Uri, RequestVerb Verb, uint64 ContentLength)
	{
		LLM_SCOPE_BYTAG(ZenDDC);
		// Strip any leading slashes because we compose the prefix and the suffix with a separating slash below
		while (Uri.StartsWith(TEXT('/')))
		{
			Uri.RightChopInline(1);
		}

		TAnsiStringBuilder<32> RequestIdHeader;
		RequestIdHeader << "UE-Request: " << RequestId++;

		const char* CommonHeaders[] = {
			GetSessionIdHeader(),
			RequestIdHeader.GetData(),
			// Strip any Expect: 100-Continue header since this just introduces latency
			"Expect:",	
			nullptr
		};

		TRACE_CPUPROFILER_EVENT_SCOPE(ZenHttp_CurlPerform);

		// Setup request options
		FString Url = FString::Printf(TEXT("%s/%s"), *Domain, *FString(Uri));
		curl_easy_setopt(Curl, CURLOPT_URL, TCHAR_TO_ANSI(*Url));

		// Setup response header buffer. If caller has not setup a response data buffer, use internal.
		WriteHeaderBufferPtr = &ResponseHeader;
		if (WriteDataBufferPtr == nullptr)
		{
			WriteDataBufferPtr = &ResponseBuffer;
		}

		if ((Verb != RequestVerb::Delete) && (Verb != RequestVerb::Get))
		{
			Headers.Add(FString::Printf(TEXT("Content-Length: %" UINT64_FMT), ContentLength));
		}

		// Build headers list
		curl_slist* CurlHeaders = nullptr;
		// Add common headers
		for (uint8 i = 0; CommonHeaders[i] != nullptr; ++i)
		{
			CurlHeaders = curl_slist_append(CurlHeaders, CommonHeaders[i]);
		}
		// Setup added headers
		for (const FString& Header : Headers)
		{
			CurlHeaders = curl_slist_append(CurlHeaders, TCHAR_TO_ANSI(*Header));
		}
		curl_easy_setopt(Curl, CURLOPT_HTTPHEADER, CurlHeaders);

		// Shots fired!
		CurlResult = curl_easy_perform(Curl);

		// Get response code
		bool bRedirected = false;
		if (CURLE_OK == curl_easy_getinfo(Curl, CURLINFO_RESPONSE_CODE, &ResponseCode))
		{
			bRedirected = (ResponseCode >= 300 && ResponseCode < 400);
		}

		LogResult(CurlResult, *FString(Uri), Verb);

		// Clean up
		curl_slist_free_all(CurlHeaders);

		return CurlResult == CURLE_OK ? Result::Success : Result::Failed;
	}

	/**
	 * Attempts to find the header from the response. Returns false if header is not present.
	 */
	bool FZenHttpRequest::GetHeader(const ANSICHAR* Header, FString& OutValue) const
	{
		check(CurlResult != CURL_LAST);  // Cannot query headers before request is sent

		const ANSICHAR* HeadersBuffer = (const ANSICHAR*)ResponseHeader.GetData();
		size_t HeaderLen = strlen(Header);

		// Find the header key in the (ANSI) response buffer. If not found we can exist immediately
		if (const ANSICHAR* Found = strstr(HeadersBuffer, Header))
		{
			const ANSICHAR* Linebreak = strchr(Found, '\r');
			const ANSICHAR* ValueStart = Found + HeaderLen + 2; //colon and space
			const size_t ValueSize = Linebreak - ValueStart;
			FUTF8ToTCHAR TCHARData(ValueStart, ValueSize);
			OutValue = FString::ConstructFromPtrSize(TCHARData.Get(), TCHARData.Length());
			return true;
		}
		return false;
	}

	FAnsiStringView FZenHttpRequest::LexResponseCodeToString(int HttpCode)
	{
		switch (HttpCode)
		{
			// 1xx Informational
		case 100: return "Continue";
		case 101: return "Switching Protocols";

			// 2xx Success
		case 200: return "OK";
		case 201: return "Created";
		case 202: return "Accepted";
		case 204: return "No Content";
		case 205: return "Reset Content";
		case 206: return "Partial Content";

			// 3xx Redirection
		case 300: return "Multiple Choices";
		case 301: return "Moved Permanently";
		case 302: return "Found";
		case 303: return "See Other";
		case 304: return "Not Modified";
		case 305: return "Use Proxy";
		case 306: return "Switch Proxy";
		case 307: return "Temporary Redirect";
		case 308: return "Permanent Redirect";

			// 4xx Client errors
		case 400: return "Bad Request";
		case 401: return "Unauthorized";
		case 402: return "Payment Required";
		case 403: return "Forbidden";
		case 404: return "Not Found";
		case 405: return "Method Not Allowed";
		case 406: return "Not Acceptable";
		case 407: return "Proxy Authentication Required";
		case 408: return "Request Timeout";
		case 409: return "Conflict";
		case 410: return "Gone";
		case 411: return "Length Required";
		case 412: return "Precondition Failed";
		case 413: return "Payload Too Large";
		case 414: return "URI Too Long";
		case 415: return "Unsupported Media Type";
		case 416: return "Range Not Satisfiable";
		case 417: return "Expectation Failed";
		case 418: return "I'm a teapot";
		case 421: return "Misdirected Request";
		case 422: return "Unprocessable Entity";
		case 423: return "Locked";
		case 424: return "Failed Dependency";
		case 425: return "Too Early";
		case 426: return "Upgrade Required";
		case 428: return "Precondition Required";
		case 429: return "Too Many Requests";
		case 431: return "Request Header Fields Too Large";

			// 5xx Server errors
		case 500: return "Internal Server Error";
		case 501: return "Not Implemented";
		case 502: return "Bad Gateway";
		case 503: return "Service Unavailable";
		case 504: return "Gateway Timeout";
		case 505: return "HTTP Version Not Supported";
		case 506: return "Variant Also Negotiates";
		case 507: return "Insufficient Storage";
		case 508: return "Loop Detected";
		case 510: return "Not Extended";
		case 511: return "Network Authentication Required";

		default: return "Unknown Result";
		}
	}


	void FZenHttpRequest::LogResult(long InResult, const TCHAR* Uri, RequestVerb Verb) const
	{
		CURLcode Result = (CURLcode)InResult;
		if (Result == CURLE_OK)
		{
			bool bSuccess = false;
			const TCHAR* VerbStr = nullptr;
			FString AdditionalInfo;

			const bool Is404 = (ResponseCode == 404);
			const bool Is2xx = (ResponseCode >= 200) && (ResponseCode <= 299);

			switch (Verb)
			{
			case RequestVerb::Head:
				bSuccess = Is2xx || Is404;
				VerbStr = TEXT("querying");
				break;
			case RequestVerb::Get:
				bSuccess = Is2xx || Is404;
				VerbStr = TEXT("fetching");
				AdditionalInfo = FString::Printf(TEXT("Received: %zu bytes."), BytesReceived);
				break;
			case RequestVerb::Put:
				bSuccess = Is2xx;
				VerbStr = TEXT("updating");
				AdditionalInfo = FString::Printf(TEXT("Sent: %zu bytes."), BytesSent);
				break;
			case RequestVerb::Post:
				bSuccess = Is2xx || Is404;
				VerbStr = TEXT("posting");
				break;
			case RequestVerb::Delete:
				bSuccess = Is2xx || Is404;
				VerbStr = TEXT("deleting");
				break;
			}

			if (bSuccess)
			{
				UE_LOGF(
					LogZenHttp,
					Verbose,
					"Finished %ls zen data (response %ld) from %ls. %ls",
					VerbStr,
					ResponseCode,
					Uri,
					*AdditionalInfo
				);
			}
			else if (bLogErrors)
			{
				// Print the response body if we got one, otherwise print header.
				FString Response = GetAnsiBufferAsString(ResponseBuffer.Num() > 0 ? ResponseBuffer : ResponseHeader);
				Response.ReplaceCharInline(TEXT('\n'), TEXT(' '));
				Response.ReplaceCharInline(TEXT('\r'), TEXT(' '));
				UE_LOGF(
					LogZenHttp,
					Error,
					"Failed %ls zen data (response %ld) from %ls. Response: %ls",
					VerbStr,
					ResponseCode,
					Uri,
					*Response
				);
			}
		}
		else if (bLogErrors)
		{
			UE_LOGF(
				LogZenHttp,
				Error,
				"Error while connecting to %ls: %ls",
				*Domain,
				ANSI_TO_TCHAR(curl_easy_strerror(Result))
			);
		}
	}

	FString FZenHttpRequest::GetAnsiBufferAsString(const TArray64<uint8>& Buffer)
	{
		// Content is NOT null-terminated; we need to specify lengths here
		FUTF8ToTCHAR TCHARData(reinterpret_cast<const ANSICHAR*>(Buffer.GetData()), IntCastChecked<int32>(Buffer.Num()));
		return FString::ConstructFromPtrSize(TCHARData.Get(), TCHARData.Length());
	}

	FString FZenHttpRequest::GetResponseAsString() const
	{
		return GetAnsiBufferAsString(ResponseBuffer);
	}

	FString FZenHttpRequest::GetDetailedResultString(FStringView Uri) const
	{
		TStringBuilder<256> SB;
		if (CurlResult != CURLE_OK)
		{
			SB << TEXTVIEW("Failed: ") << GetError();
			SB << TEXTVIEW(" (") << CurlResult << TEXTVIEW(") ");
		}
		SB << TEXTVIEW("Url: ") << Uri;

		if (ResponseCode != 0)
		{
			SB << TEXTVIEW(" Response: ") << LexResponseCodeToString(ResponseCode) << TEXTVIEW(" (") << ResponseCode << TEXTVIEW(")");
		}

		FString ResponseString = GetResponseAsString();
		if (!ResponseString.IsEmpty())
		{
			SB << TEXTVIEW(" \"") << ResponseString << TEXTVIEW("\"");
		}
		else if (int64 PayloadSize = ResponseBuffer.Num(); PayloadSize > 0)
		{
			SB << TEXTVIEW(" ") << PayloadSize << TEXTVIEW(" bytes");
		}
		return SB.ToString();
	}

	FCbObjectView FZenHttpRequest::GetResponseAsObject() const
	{
		return FCbObjectView(ResponseBuffer.GetData());
	}

	size_t FZenHttpRequest::FStatics::StaticDebugCallback(CURL* Handle, curl_infotype DebugInfoType, char* DebugInfo, size_t DebugInfoSize, void* UserData)
	{
		FZenHttpRequest* Request = static_cast<FZenHttpRequest*>(UserData);

		switch (DebugInfoType)
		{
			case CURLINFO_TEXT:
				{
					// Truncate at 1023 characters. This is just an arbitrary number based on a buffer size seen in
					// the libcurl code.
					DebugInfoSize = FMath::Min(DebugInfoSize, (size_t)1023);

					// Calculate the actual length of the string due to incorrect use of snprintf() in lib/vtls/openssl.c.
					char* FoundNulPtr = (char*)memchr(DebugInfo, 0, DebugInfoSize);
					int CalculatedSize = FoundNulPtr != nullptr ? FoundNulPtr - DebugInfo : DebugInfoSize;

					auto ConvertedString = StringCast<TCHAR>(static_cast<const ANSICHAR*>(DebugInfo), CalculatedSize);
					FString DebugText = FString::ConstructFromPtrSize(ConvertedString.Get(), ConvertedString.Length());
					DebugText.ReplaceInline(TEXT("\n"), TEXT(""), ESearchCase::CaseSensitive);
					DebugText.ReplaceInline(TEXT("\r"), TEXT(""), ESearchCase::CaseSensitive);
					UE_LOGF(LogZenHttp, VeryVerbose, "CURL %p: '%ls'", Request, *DebugText);
				}
				break;

			case CURLINFO_HEADER_IN:
				UE_LOGF(LogZenHttp, VeryVerbose, "CURL %p: Received header (%zd bytes)", Request, DebugInfoSize);
				UE_LOGF(LogZenHttp, VeryVerbose, "CURL HEADER <<< %*s", uint32(DebugInfoSize), DebugInfo);
				break;

			case CURLINFO_HEADER_OUT:
				UE_LOGF(LogZenHttp, VeryVerbose, "CURL %p: Send header (%zd bytes)", Request, DebugInfoSize);
				UE_LOGF(LogZenHttp, VeryVerbose, "CURL HEADER >>> %*s", uint32(DebugInfoSize), DebugInfo);
				break;

			case CURLINFO_DATA_IN:
				UE_LOGF(LogZenHttp, VeryVerbose, "CURL %p: Received data (%zd bytes)", Request, DebugInfoSize);
				break;

			case CURLINFO_DATA_OUT:
				UE_LOGF(LogZenHttp, VeryVerbose, "CURL %p: Sent data (%zd bytes)", Request, DebugInfoSize);
				break;

			case CURLINFO_SSL_DATA_IN:
				UE_LOGF(LogZenHttp, VeryVerbose, "CURL %p: Received SSL data (%zd bytes)", Request, DebugInfoSize);
				break;

			case CURLINFO_SSL_DATA_OUT:
				UE_LOGF(LogZenHttp, VeryVerbose, "CURL %p: Sent SSL data (%zd bytes)", Request, DebugInfoSize);
				break;
		}

		return 0;
	}

	size_t FZenHttpRequest::FStatics::StaticReadFn(void* Ptr, size_t SizeInBlocks, size_t BlockSizeInBytes, void* UserData)
	{
		FZenHttpRequest* Request = static_cast<FZenHttpRequest*>(UserData);
		check(Request->ReadDataView);
		const FCompositeBuffer& ReadDataView = *Request->ReadDataView;

		const size_t Offset = Request->BytesSent;
		const size_t ReadSize = FMath::Min((size_t)ReadDataView.GetSize() - Offset, SizeInBlocks * BlockSizeInBytes);
		check(ReadDataView.GetSize() >= Offset + ReadSize);

		Memcpy(Ptr, ReadDataView, Offset, ReadSize);
		Request->BytesSent += ReadSize;
		return ReadSize;
		}

	size_t FZenHttpRequest::FStatics::StaticWriteHeaderFn(void* Ptr, size_t SizeInBlocks, size_t BlockSizeInBytes, void* UserData)
	{
		FZenHttpRequest* Request = static_cast<FZenHttpRequest*>(UserData);
		const size_t WriteSize = SizeInBlocks * BlockSizeInBytes;
		TArray64<uint8>* WriteHeaderBufferPtr = Request->WriteHeaderBufferPtr;
		if (WriteHeaderBufferPtr && WriteSize > 0)
		{
			const size_t CurrentBufferLength = WriteHeaderBufferPtr->Num();
			if (CurrentBufferLength > 0)
			{
				// Remove the previous zero termination
				(*WriteHeaderBufferPtr)[CurrentBufferLength - 1] = ' ';
			}

			// Write the header
			WriteHeaderBufferPtr->Append((const uint8*)Ptr, WriteSize + 1);
			(*WriteHeaderBufferPtr)[WriteHeaderBufferPtr->Num() - 1] = 0; // Zero terminate string
			return WriteSize;
		}
		return 0;
	}

	size_t FZenHttpRequest::FStatics::StaticWriteBodyFn(void* Ptr, size_t SizeInBlocks, size_t BlockSizeInBytes, void* UserData)
	{
		FZenHttpRequest* Request = static_cast<FZenHttpRequest*>(UserData);
		const int64 WriteSize = IntCastChecked<int64>(SizeInBlocks * BlockSizeInBytes);
		TArray64<uint8>* WriteDataBufferPtr = Request->WriteDataBufferPtr;

		if (WriteDataBufferPtr && WriteSize > 0)
		{
			// If this is the first part of the body being received, try to reserve 
			// memory if content length is defined in the header.
			if (Request->BytesReceived == 0 && Request->WriteHeaderBufferPtr)
			{
				static const ANSICHAR* ContentLengthHeaderStr = "Content-Length: ";
				const ANSICHAR* Header = (const ANSICHAR*)Request->WriteHeaderBufferPtr->GetData();

				if (const ANSICHAR* ContentLengthHeader = FCStringAnsi::Strstr(Header, ContentLengthHeaderStr))
				{
					int64 ContentLength = FCStringAnsi::Atoi64(ContentLengthHeader + strlen(ContentLengthHeaderStr));
					if (ContentLength > 0)
					{
						WriteDataBufferPtr->Reserve(ContentLength);
					}
				}
			}

			// Write to the target buffer
			WriteDataBufferPtr->Append((const uint8*)Ptr, WriteSize);
			Request->BytesReceived += WriteSize;
			return WriteSize;
		}

		return 0;
	}

	size_t FZenHttpRequest::FStatics::StaticSeekFn(void* UserData, curl_off_t Offset, int Origin)
	{
		FZenHttpRequest* Request = static_cast<FZenHttpRequest*>(UserData);
		size_t NewPosition = 0;
		uint64 ReadDataSize = Request->ReadDataView ? Request->ReadDataView->GetSize() : 0;

		switch (Origin)
		{
			case SEEK_SET: 
				NewPosition = Offset; 
				break;
			case SEEK_CUR: 
				NewPosition = Request->BytesSent + Offset; 
				break;
			case SEEK_END: 
				NewPosition = ReadDataSize + Offset; 
				break;
		}

		// Make sure we don't seek outside of the buffer
		if (NewPosition < 0 || NewPosition >= ReadDataSize)
		{
			return CURL_SEEKFUNC_FAIL;
		}

		// Update the used offset
		Request->BytesSent = NewPosition;
		return CURL_SEEKFUNC_OK;
	}

#if PLATFORM_WINDOWS
	int FZenHttpRequest::FStatics::StaticSockoptFn(void* UserData, curl_socket_t CurlFd, curlsocktype Purpose)
	{
		// On Windows, loopback connections can take advantage of a faster code path optionally with this flag.
		// This must be used by both the client and server side, and is only effective in the absence of
		// Windows Filtering Platform (WFP) callouts which can be installed by security software.
		// https://docs.microsoft.com/en-us/windows/win32/winsock/sio-loopback-fast-path
		int LoopbackOptionValue = 1;
		DWORD OptionNumberOfBytesReturned = 0;
		WSAIoctl(CurlFd, SIO_LOOPBACK_FAST_PATH, &LoopbackOptionValue, sizeof(LoopbackOptionValue), NULL, 0, &OptionNumberOfBytesReturned, 0, 0);
		return CURL_SOCKOPT_OK;
	}
#endif //PLATFORM_WINDOWS

	//////////////////////////////////////////////////////////////////////////

	FZenHttpRequestPool::FZenHttpRequestPool(const FZenServiceEndpoint& Endpoint, uint32 PoolEntryCount)
	{
		Pool.SetNum(PoolEntryCount);

		for (uint8 i = 0; i < Pool.Num(); ++i)
		{
			Pool[i].IsAllocated = 0u;
			Pool[i].Request = new FZenHttpRequest(Endpoint, true /* bLogErrors */);
		}
	}

	FZenHttpRequestPool::~FZenHttpRequestPool()
	{
		for (uint8 i = 0; i < Pool.Num(); ++i)
		{
			// No requests should be in use by now.
			check(Pool[i].IsAllocated.load(std::memory_order_relaxed) == 0u);

			delete Pool[i].Request;
		}
	}

	/** Block until a request is free
	  *
	  * Once a request has been returned it is owned by the caller and
	  * it needs to release it to the pool when work has been completed
	  *
	  * @return Usable request instance.
	  */
	FZenHttpRequest* FZenHttpRequestPool::WaitForFreeRequest()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ZenHttp_WaitForConnPool);

		while (true)
		{
			for (uint8 i = 0; i < Pool.Num(); ++i)
			{
				if (!Pool[i].IsAllocated.load(std::memory_order_relaxed))
				{
					uint8 Expected = 0u;
					if (Pool[i].IsAllocated.compare_exchange_strong(Expected, 1u))
					{
						return Pool[i].Request;
					}
				}
			}

			// This should use a better mechanism like condition variables / events
			FPlatformProcess::Sleep(UE_ZENDDC_BACKEND_WAIT_INTERVAL);
		}
	}

	/**
	  * Release request to the pool.
	  * @param Request Request that should be freed. Note that any buffer owned by the request can now be reset.
	  */
	void FZenHttpRequestPool::ReleaseRequestToPool(FZenHttpRequest* Request)
	{
		for (uint8 i = 0; i < Pool.Num(); ++i)
		{
			if (Pool[i].Request == Request)
			{
				Request->Reset();
				uint8 Expected = 1u;
				Pool[i].IsAllocated.compare_exchange_strong(Expected, 0u);
				return;
			}
		}
		check(false);
	}
}
