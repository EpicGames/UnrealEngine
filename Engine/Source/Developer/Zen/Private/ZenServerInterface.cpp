// Copyright Epic Games, Inc. All Rights Reserved.

#include "Experimental/ZenServerInterface.h"

#include "ZenBackendUtils.h"
#include "ZenSerialization.h"
#include "ZenServerHttp.h"
#include "ZenServerState.h"
#include "ZenVersion.h"

#include "AnalyticsEventAttribute.h"
#include "Async/Async.h"
#include "Async/UniqueLock.h"
#include "Containers/AnsiString.h"
#include "Dom/JsonValue.h"
#include "HAL/FileManager.h"
#include "HAL/Platform.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformProcess.h"
#include "Logging/LogScopedCategoryAndVerbosityOverride.h"
#include "Logging/StructuredLog.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigContext.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/ScopeExit.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryValue.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "String/LexFromString.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Misc/Guid.h"
#include "Policies/CondensedJsonPrintPolicy.h"

#if PLATFORM_WINDOWS
#	include "Windows/AllowWindowsPlatformTypes.h"
#	include <Windows.h>
#	include "Windows/HideWindowsPlatformTypes.h"
#	include <dbghelp.h>
#	include <accctrl.h>
#	include <AclAPI.h>
#endif

#if PLATFORM_MAC
#	include "Mac/MacSystemIncludes.h"
#endif

#define ALLOW_SETTINGS_OVERRIDE_FROM_COMMANDLINE			(UE_SERVER || !(UE_BUILD_SHIPPING))

namespace UE::Zen
{

DEFINE_LOG_CATEGORY_STATIC(LogZenServiceInstance, Log, All);

FZenServiceEndpoint::FZenServiceEndpoint(FStringView InName, uint16 InPort)
: Name(InName)
, Port(InPort)
{
	if (InName.EndsWith(TEXT(".sock")))
	{
		checkf(Port == 0, TEXT("Host name appears to be a Unix socket path (contains slashes) and therefore port should be 0"));
		SocketType = ESocketType::Unix;
	}
}

FStringView FZenServiceEndpoint::GetHostName() const
{
	return (GetSocketType() == ESocketType::Tcp) ? Name : TEXTVIEW("localhost");
}

FString FZenServiceEndpoint::GetURL() const
{
	TStringBuilder<128> Builder;
	Builder << TEXTVIEW("http://");
	Builder << GetHostName();
	if (Port > 0)
	{
		Builder << TEXT(":") << Port;
	}
	return FString(Builder);
}

struct FZenServiceLink
{
	FString ServicePath;
	FString UtilityPath;
	FZenVersion Version;

	operator bool() const
	{
		return !ServicePath.IsEmpty() && !UtilityPath.IsEmpty() && Version;
	}

	static FZenServiceLink Read(const FString& Filename)
	{
		FString JsonText;
		if (FFileHelper::LoadFileToString(JsonText, *Filename))
		{
			TSharedPtr<FJsonObject> JsonObject;
			TSharedRef<TJsonReader<TCHAR>> Reader = TJsonReaderFactory<TCHAR>::Create(JsonText);
			if (FJsonSerializer::Deserialize(Reader, JsonObject) && JsonObject.IsValid())
			{
				FString ServicePath = JsonObject->GetFieldUntyped(TEXT("ServicePath"))->AsString();
				FString UtilityPath = JsonObject->GetFieldUntyped(TEXT("UtilityPath"))->AsString();
				auto VersionObject = JsonObject->GetFieldUntyped(TEXT("Version"))->AsObject();
				if (VersionObject)
				{
					uint32_t MajorVersion = static_cast<uint32_t>(VersionObject->TryGetField(TEXT("Major"))->AsNumber());
					uint32_t MinorVersion = static_cast<uint32_t>(VersionObject->TryGetField(TEXT("Minor"))->AsNumber());
					uint32_t PatchVersion = static_cast<uint32_t>(VersionObject->TryGetField(TEXT("Patch"))->AsNumber());
					FString Details = VersionObject->TryGetField(TEXT("Details"))->AsString();
					return FZenServiceLink{
						.ServicePath = ServicePath,
						.UtilityPath = UtilityPath,
						.Version = FZenVersion{
							.MajorVersion = MajorVersion,
							.MinorVersion = MinorVersion,
							.PatchVersion = PatchVersion,
							.Details = Details}
					};
				}
			}
		}
		return {};
	}

	static bool Write(const FZenServiceLink& Link, const FString& Filename)
	{
		FString JsonTcharText;
		TSharedRef<TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>> Writer = TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>::Create(&JsonTcharText);
		Writer->WriteObjectStart();

		Writer->WriteValue(TEXT("ServicePath"), Link.ServicePath);
		Writer->WriteValue(TEXT("UtilityPath"), Link.UtilityPath);

		Writer->WriteObjectStart(TEXT("Version"));
		Writer->WriteValue(TEXT("Major"), Link.Version.MajorVersion);
		Writer->WriteValue(TEXT("Minor"), Link.Version.MinorVersion);
		Writer->WriteValue(TEXT("Patch"), Link.Version.PatchVersion);
		Writer->WriteValue(TEXT("Details"), Link.Version.Details);
		Writer->WriteObjectEnd();

		Writer->WriteObjectEnd();
		Writer->Close();

		if (!FFileHelper::SaveStringToFile(JsonTcharText, *Filename))
		{
			return false;
		}

		return true;
	}
};

static FString
GetLocalZenRootPath()
{
	return FPaths::ConvertRelativePathToFull(FPaths::Combine(FPlatformProcess::UserSettingsDir(), *FApp::GetEpicProductIdentifier(), TEXT("Common")) + TEXT("/"));
}

static FString
GetServiceExecutableName()
{
	return
#if PLATFORM_WINDOWS
		TEXT("zenserver.exe")
#else
		TEXT("zenserver")
#endif
		;
}

static FString
GetUtilityExecutableName()
{
	return
#if PLATFORM_WINDOWS
		TEXT("zen.exe")
#else
		TEXT("zen")
#endif
		;
}

static FString
GetOidcTokenExecutableName()
{
	return
#if PLATFORM_WINDOWS
		TEXT("OidcToken.exe")
#else
		TEXT("OidcToken")
#endif
		;
}

static FString
GetSystemInstallPath(const FSystemServiceSettings& InSettings)
{
	FString InstallPath = InSettings.InstallPath.IsEmpty() ? FPaths::Combine(FPlatformProcess::ApplicationSettingsDir(), TEXT("Zen/")) : InSettings.InstallPath;
	return FPaths::ConvertRelativePathToFull(InstallPath);
}

static FString
GetSystemServiceInstallPath(const FSystemServiceSettings& InSettings)
{
	return FPaths::ConvertRelativePathToFull(GetSystemInstallPath(InSettings), GetServiceExecutableName());
}

static FString
GetSystemUtilityInstallPath(const FSystemServiceSettings& InSettings)
{
	return FPaths::ConvertRelativePathToFull(GetSystemInstallPath(InSettings), GetUtilityExecutableName());
}

static FString
GetSystemServiceVersionCachePath(const FSystemServiceSettings& InSettings)
{
	FString InstallUtilityPath = GetSystemUtilityInstallPath(InSettings);
	FString InstallVersionCache = FPaths::SetExtension(InstallUtilityPath, TEXT("version"));
	return InstallVersionCache;
}

namespace Private
{
// For testing purposes. See comments on GetLocalInstallPathOverride() in ZenServerInterface.h.
static FString LocalInstallPathOverride;

void
SetLocalInstallPathOverride(const FString& Path)
{
	checkf(LocalInstallPathOverride.IsEmpty() || Path.IsEmpty(), TEXT("Local Zen install path override already set to %s"), *LocalInstallPathOverride);
	UE_LOGF(LogZenServiceInstance, Display, "Setting local install path override to %ls", *Path);
	LocalInstallPathOverride = Path;
}

FString
GetLocalInstallPathOverride()
{
	return LocalInstallPathOverride;
}

}

static FString
GetLocalInstallPath()
{
	return Private::GetLocalInstallPathOverride().IsEmpty() ? FPaths::ConvertRelativePathToFull(FPaths::Combine(GetLocalZenRootPath(), TEXT("Zen\\Install"))) : Private::GetLocalInstallPathOverride();
}

static FString
GetServiceLinkPath()
{
	return FPaths::Combine(GetLocalInstallPath(), TEXT("zen.link"));
}

static FString
GetServicePluginsConfigPath()
{
	const int PluginsConfigVersion = 1;
	return FPaths::Combine(GetLocalInstallPath(), FString::Printf(TEXT("zen_plugins_v%d.json"), PluginsConfigVersion));
}

static FString
GetServicePluginsVersionInfoPath()
{
	return FPaths::Combine(GetLocalInstallPath(), TEXT("zen_plugin_versions.json"));
}

static void
CleanOutOfDateServicePluginConfigs()
{
	IFileManager::Get().IterateDirectory(*GetLocalInstallPath(), [](const TCHAR* Pathname, bool bIsDirectory)
	{
		if (!bIsDirectory)
		{
			FString Path = Pathname;
			FString FileName = FPaths::GetCleanFilename(Path);

			if (FileName.EndsWith(TEXT(".json")) && FileName.StartsWith("zen_plugins") &&
				FileName != FPaths::GetCleanFilename(GetServicePluginsConfigPath()))
			{
				IFileManager::Get().Delete(Pathname, false /*RequireExists*/, false /*EvenReadOnly*/, true /*Quiet*/);
			}
		}
		return true;
	});
}

static FString
GetSecurityConfigPath(const FString& InstallPath)
{
	return FPaths::Combine(InstallPath, TEXT("security-config.json"));
}

static FString
BuildSecurityConfigJson(const TSharedPtr<FJsonObject>& AuthObject)
{
	FString Json;
	TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&Json);
	Writer->WriteObjectStart();
	Writer->WriteObjectStart(TEXT("http"));
	Writer->WriteObjectStart(TEXT("root"));
	Writer->WriteObjectStart(TEXT("filter"));

	FString AuthType;
	if (AuthObject->TryGetStringField(TEXT("type"), AuthType) && AuthType == TEXT("password"))
	{
		FString Username;
		FString Password;
		AuthObject->TryGetStringField(TEXT("username"), Username);
		AuthObject->TryGetStringField(TEXT("password"), Password);

		Writer->WriteValue(TEXT("type"), TEXT("password"));
		Writer->WriteObjectStart(TEXT("config"));
		Writer->WriteObjectStart(TEXT("basic"));
		Writer->WriteValue(TEXT("username"), Username);
		Writer->WriteValue(TEXT("password"), Password);
		Writer->WriteObjectEnd(); // basic
		Writer->WriteValue(TEXT("protect-machine-local-requests"), false);
		Writer->WriteObjectEnd(); // config
	}

	Writer->WriteObjectEnd(); // filter
	Writer->WriteObjectEnd(); // root
	Writer->WriteObjectEnd(); // http
	Writer->WriteObjectEnd();
	Writer->Close();
	return Json;
}

bool
GetLocalServiceHostAuth(FString& OutAuthJson)
{
	return FPlatformMisc::GetStoredValue(TEXT("Epic Games"), TEXT("Zen"), TEXT("HostAuth"), OutAuthJson)
		&& !OutAuthJson.IsEmpty();
}

static void
SetLocalServiceHostAuth(const FString& AuthJson)
{
	FPlatformMisc::SetStoredValue(TEXT("Epic Games"), TEXT("Zen"), TEXT("HostAuth"), AuthJson);
}

static void
ClearLocalServiceHostAuth()
{
	FPlatformMisc::DeleteStoredValue(TEXT("Epic Games"), TEXT("Zen"), TEXT("HostAuth"));
}

static bool
GetGeneratedStaticKey(FString& OutKey)
{
	return FPlatformMisc::GetStoredValue(TEXT("Epic Games"), TEXT("Zen"), TEXT("GeneratedStaticKey"), OutKey)
		&& !OutKey.IsEmpty();
}

static void
SetGeneratedStaticKey(const FString& Key)
{
	FPlatformMisc::SetStoredValue(TEXT("Epic Games"), TEXT("Zen"), TEXT("GeneratedStaticKey"), Key);
}

// Callers must hold a system-wide critical section that serializes access to
// the security config file for the given InstallPath (e.g. ZenServerLaunch
// for auto-launch, ZenServerInstall for system service).
static bool
ConditionalUpdateSecurityConfig(ERemoteNetworkService RemoteNetworkService, const FString& InstallPath)
{
	const bool bEnabled = (RemoteNetworkService == ERemoteNetworkService::GeneratedStaticKey);
	const FString SecurityConfigPath = GetSecurityConfigPath(InstallPath);
	const bool bFileExists = IFileManager::Get().FileExists(*SecurityConfigPath);

	if (!bEnabled)
	{
		// Wipe HostAuth so consumers don't write stale auth data to ue.projectstore.
		// GeneratedStaticKeyAuth is left intact so switching back to GeneratedStaticKey
		// mode reuses the same key rather than generating a new one.
		ClearLocalServiceHostAuth();

		if (bFileExists)
		{
			IFileManager::Get().Delete(*SecurityConfigPath, false /*RequireExists*/, false /*EvenReadOnly*/, true /*Quiet*/);
			return true;
		}
		return false;
	}

	// Retrieve or generate the static key. The key is persisted in GeneratedStaticKey
	// so it survives mode switches to None/Unsecured and back.
	FString GeneratedStaticKey;
	if (!GetGeneratedStaticKey(GeneratedStaticKey))
	{
		FString NewKey = FGuid::NewGuid().ToString(EGuidFormats::Digits);
		SetGeneratedStaticKey(NewKey);
		GeneratedStaticKey = NewKey;
	}

	FString NewAuthJson;
	TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> Writer = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&NewAuthJson);
	Writer->WriteObjectStart();
	Writer->WriteValue(TEXT("type"), TEXT("password"));
	Writer->WriteValue(TEXT("username"), TEXT("ue"));
	Writer->WriteValue(TEXT("password"), GeneratedStaticKey);
	Writer->WriteObjectEnd();
	Writer->Close();

	FString AuthJson;
	if (!GetLocalServiceHostAuth(AuthJson) || AuthJson != NewAuthJson)
	{
		// Publish to HostAuth so consumers (C#, Python, etc.) can read it.
		SetLocalServiceHostAuth(NewAuthJson);
		AuthJson = NewAuthJson;
	}

	TSharedPtr<FJsonObject> AuthObject;
	TSharedRef<TJsonReader<>> AuthReader = TJsonReaderFactory<>::Create(AuthJson);
	if (!FJsonSerializer::Deserialize(AuthReader, AuthObject) || !AuthObject.IsValid())
	{
		return false;
	}

	FString ExpectedJson = BuildSecurityConfigJson(AuthObject);
	FString ExistingJson;
	if (FFileHelper::LoadFileToString(ExistingJson, *SecurityConfigPath) && ExistingJson == ExpectedJson)
	{
		return false;
	}

	UE_LOGF(LogZenServiceInstance, Display, "Writing security configuration to '%ls'", *SecurityConfigPath);
	if (!FFileHelper::SaveStringToFile(ExpectedJson, *SecurityConfigPath, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM))
	{
		UE_LOGF(LogZenServiceInstance, Error, "Failed to write security configuration to '%ls'", *SecurityConfigPath);
	}
	return true;
}

static FString
GetServiceCopyInstallPath()
{
	return FPaths::ConvertRelativePathToFull(FPaths::Combine(GetLocalInstallPath(), GetServiceExecutableName()));
}

static FString
GetUtilityCopyInstallPath()
{
	return FPaths::ConvertRelativePathToFull(FPaths::Combine(GetLocalInstallPath(), GetUtilityExecutableName()));
}

static FString
GetInstallVersionCachePath()
{
	FString InstallUtilityPath = GetUtilityCopyInstallPath();
	FString InstallVersionCache = FPaths::SetExtension(InstallUtilityPath, TEXT("version"));
	return InstallVersionCache;
}

static FString
GetInTreeVersionCache()
{
	return FPaths::ConvertRelativePathToFull(FPaths::Combine(FPaths::EngineSavedDir(), TEXT("Zen"), TEXT("zen.version")));
}

static FString
GetServiceRunContextPath()
{
	return FPaths::SetExtension(GetServiceCopyInstallPath(), TEXT(".runcontext"));
}

static FString
GetInTreeUtilityPath()
{
	return FPaths::ConvertRelativePathToFull(FPlatformProcess::GenerateApplicationPath(TEXT("zen"), EBuildConfiguration::Development));
}

static FString
GetInTreeServicePath()
{
	return FPaths::ConvertRelativePathToFull(FPlatformProcess::GenerateApplicationPath(TEXT("zenserver"), EBuildConfiguration::Development));
}

static FString
GetInTreeCrashpadHandlerFilePath()
{
	return FPaths::ConvertRelativePathToFull(FPlatformProcess::GenerateApplicationPath(TEXT("crashpad_handler"), EBuildConfiguration::Development));
}

static FString
GetInstallCrashpadHandlerFilePath(const FString& InTreePath)
{
	return FPaths::ConvertRelativePathToFull(FPaths::Combine(GetLocalInstallPath(), FString(FPathViews::GetCleanFilename(InTreePath))));
}

static FString
GetInTreeOidcTokenPath()
{
	const FString PlatformDir =
#if PLATFORM_WINDOWS
		TEXT("win-x64")
#elif PLATFORM_MAC
		TEXT("osx-x64")
#else	
		TEXT("linux-x64")
#endif
	;

	return FPaths::ConvertRelativePathToFull(FPaths::Combine(FPaths::EngineDir(), TEXT("Binaries"), TEXT("DotNet"), TEXT("OidcToken"), PlatformDir, GetOidcTokenExecutableName()));
}

static FString
GetInstallOidcTokenPath()
{
	return FPaths::ConvertRelativePathToFull(FPaths::Combine(GetLocalInstallPath(), GetOidcTokenExecutableName()));
}

#if PLATFORM_WINDOWS

FString WriteMiniDump(HANDLE ProcessHandle, const FString& ExeName, const FString& OutputFolderPath)
{
	const FDateTime Now = FDateTime::Now();
	const FString Timestamp = Now.ToString(TEXT("%Y%m%d_%H%M%S"));
	FString FileName = FString::Printf(TEXT("%s_Dump_%s.dmp"), *ExeName, *Timestamp);

	FString DumpPath = FPaths::Combine(OutputFolderPath, FileName);

	HANDLE FileHandle = ::CreateFile(*DumpPath, GENERIC_WRITE, FILE_SHARE_WRITE, NULL, CREATE_ALWAYS, FILE_ATTRIBUTE_NORMAL, NULL);
	if (FileHandle != INVALID_HANDLE_VALUE)
	{
		MINIDUMP_TYPE MinidumpType = (MINIDUMP_TYPE)(MiniDumpWithFullMemory | MiniDumpWithFullMemoryInfo | MiniDumpWithHandleData | MiniDumpWithThreadInfo);

		const BOOL Result = MiniDumpWriteDump(ProcessHandle, GetProcessId(ProcessHandle), FileHandle, MinidumpType, NULL, NULL, NULL);
		CloseHandle(FileHandle);
		if (Result)
		{
			return DumpPath;
		}
	}
	return FString{};
}

#endif // PLATFORM_WINDOWS

static bool RunZenUtility(const FString& UtilityPath, const FString& CommandLine, FString& Output, int& ReturnCode)
{
	FString AbsoluteUtilityPath = UtilityPath;

	// If we're running zen on Linux/Mac using 'sudo' or 'pkexec' we don't want to make those paths relative.
	// In that case UtilityPath is 'pkexec'/'sudo' and the full path to the zen binary is already part of
	// CommandLine.
	if (AbsoluteUtilityPath.StartsWith(TEXT("zen")))
	{
		AbsoluteUtilityPath = FPaths::ConvertRelativePathToFull(UtilityPath);
		FPaths::MakePlatformFilename(AbsoluteUtilityPath);
	}

	bool bLaunchDetached = true;
	bool bLaunchHidden = true;
	bool bLaunchReallyHidden = true;
	uint32* OutProcessID = nullptr;
	int32 PriorityModifier = 0;
	void* PipeWriteChild = nullptr;
	void* PipeReadChild = nullptr;

	UE_LOGF(LogZenServiceInstance, Display, "Launching zen utility '%ls %ls'.", *AbsoluteUtilityPath, *CommandLine);

	if (!FPlatformProcess::CreatePipe(PipeReadChild, PipeWriteChild))
	{
		UE_LOGF(LogZenServiceInstance, Warning, "Failed to create pipes for Zen child process - output may be incomplete");
	}

	FProcHandle ZenUtilProc = FPlatformProcess::CreateProc(
		*AbsoluteUtilityPath,
		*CommandLine,
		bLaunchDetached,
		bLaunchHidden,
		bLaunchReallyHidden,
		OutProcessID,
		PriorityModifier,
		nullptr,
		PipeWriteChild);

	if (!ZenUtilProc.IsValid())
	{
		UE_LOGF(LogZenServiceInstance, Warning, "Failed to launch zen utility '%ls %ls'", *AbsoluteUtilityPath, *CommandLine);
		return false;
	}

	bool DumpWritten = false;

	double MaxWaitDuration = FApp::IsUnattended() ? 25.0 : 8.0;

#if PLATFORM_LINUX
	// Linux can require interactive password entry when installing system services, so increase the timeout
	// if we're running this under 'sudo' or 'pkexec' rather than directly shelling out to zen
	MaxWaitDuration = (UtilityPath.StartsWith("zen") ? MaxWaitDuration : 30.0);
#endif

	const uint64 StartTime = FPlatformTime::Cycles64();
	while (FPlatformProcess::IsProcRunning(ZenUtilProc))
	{
		double Duration = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);
		if (Duration >= 2.0)
		{
			if (Duration > 15.0)
			{
#if PLATFORM_WINDOWS
				if (!DumpWritten)
				{
					const FString ZenExeName = FPaths::GetBaseFilename(UtilityPath);
					const FString AutomationToolSavedDir = FPaths::Combine(FPaths::ConvertRelativePathToFull(FPaths::EngineDir()), TEXT("Programs"), TEXT("AutomationTool"), TEXT("Saved"), TEXT("Logs"));
					const FString DumpPath = WriteMiniDump(ZenUtilProc.Get(), ZenExeName, AutomationToolSavedDir);
					if (DumpPath.IsEmpty())
					{
						UE_LOGF(LogZenServiceInstance, Warning, "Zen utility process has been running for %.1lf seconds without completing, failed to write minidump for zen utility to %ls", Duration, *AutomationToolSavedDir);
					}
					else
					{
						UE_LOGF(LogZenServiceInstance, Warning, "Zen utility process has been running for %.1lf seconds without completing, wrote minidump for zen utility to %ls", Duration, *DumpPath);
					}
					DumpWritten = true;
				}
#endif // PLATFORM_WINDOWS
			}

			if (Duration >= MaxWaitDuration)
			{
				UE_LOGF(LogZenServiceInstance, Warning, "Zen utility process has been running for %.1lf seconds without completing, aborting.", Duration);
				FPlatformProcess::TerminateProc(ZenUtilProc);
				FPlatformProcess::CloseProc(ZenUtilProc);
				return false;
			}
			UE_LOGF(LogZenServiceInstance, Display, "Zen utility process has been running for %.1lf seconds without completing, still waiting...", Duration);
			FPlatformProcess::Sleep(1.0f);
		}
		else if (Duration >= 0.2f)
		{
			FPlatformProcess::Sleep(0.1f);
		}
		else
		{
			FPlatformProcess::Sleep(0.05f);
		}
	}

	double Duration = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);

	bool GetReturnCodeSucceeded = FPlatformProcess::GetProcReturnCode(ZenUtilProc, &ReturnCode);
	Output = FPlatformProcess::ReadPipe(PipeReadChild);

	if (Output.IsEmpty())
	{
		UE_LOGF(LogZenServiceInstance, Warning, "Zen utility process completed after %.3lf seconds but no output was available", Duration);
	}

	FPlatformProcess::CloseProc(ZenUtilProc);

	if (GetReturnCodeSucceeded)
	{
		return true;
	}
	else
	{
		UE_LOGF(LogZenServiceInstance, Warning, "Zen utility process completed after %.3lf seconds but failed to fetch return code of process", Duration);
		return false;
	}
}

static bool RunZenUtilityElevated(const FString& UtilityPath, const FString& CommandLine, FString& Output, int& ReturnCode, const FString& PassthroughArgs = TEXT(""))
{
	FString ElevatedCommandLine;
	FString ElevatedUtilityPath;

#if PLATFORM_WINDOWS
	ElevatedUtilityPath = UtilityPath;
	ElevatedCommandLine = CommandLine + TEXT(" --allow-elevation ");
#endif
#if PLATFORM_LINUX
	ElevatedCommandLine = UtilityPath + TEXT(" ") + CommandLine;
	ElevatedUtilityPath = TEXT("pkexec");
#endif
#if PLATFORM_MAC
	ElevatedUtilityPath = UtilityPath;
	ElevatedCommandLine = CommandLine;
#endif

	if (PassthroughArgs != TEXT(""))
	{
		ElevatedCommandLine += TEXT(" -- ") + PassthroughArgs;
	}

	return RunZenUtility(ElevatedUtilityPath, ElevatedCommandLine, Output, ReturnCode);
}

static bool GetZenVersion(const FString& UtilityPath, const FString& ServicePath, FZenVersion& OutVersion)
{
	FString TempOutputFilePath = FPaths::CreateTempFilename(FPlatformProcess::UserTempDir(), TEXT("zenversion"));
	FPaths::MakePlatformFilename(TempOutputFilePath);

	ON_SCOPE_EXIT
	{
		IFileManager& FileManager = IFileManager::Get();
		bool DeleteResult = FileManager.Delete(*TempOutputFilePath, false, false, true);
	};

	TStringBuilder<512> Args;
	Args << TEXTVIEW("version --detailed --output-path \"");
	Args << TempOutputFilePath;
	Args << TEXTVIEW("\"");

	FString Output;
	int ReturnCode;
	if (!RunZenUtility(UtilityPath, *Args, Output, ReturnCode) || ReturnCode != 0)
	{
		UE_LOGF(LogZenServiceInstance, Warning, "Zen utility process completed but failed with error code %d", ReturnCode);
		return false;
	}

	if (!FPaths::FileExists(TempOutputFilePath))
	{
		UE_LOGF(LogZenServiceInstance, Warning, "Zen utility process completed successfully but failed to produce an output file to '%ls'", *TempOutputFilePath);
		return false;
	}

	FString VersionString;
	if (!FFileHelper::LoadFileToString(VersionString, *TempOutputFilePath))
	{
		UE_LOGF(LogZenServiceInstance, Warning, "Zen utility process completed successfully but failed to read output file at '%ls'", *TempOutputFilePath);
		return false;
	}

	FString VersionOutputString = VersionString.TrimStartAndEnd();

	if (!OutVersion.TryParse(*VersionOutputString))
	{
		UE_LOGF(LogZenServiceInstance, Warning, "Zen utility process completed successfully but output file is malformed : '%ls'", *VersionString);
		return false;
	}

	UE_LOGF(LogZenServiceInstance, Display, "Zen utility process completed successfully, version: '%ls'", *VersionString);
	return true;
}

static FZenVersion
GetZenVersion(const FString& UtilityPath, const FString& ServicePath, const FString& VersionCachePath)
{
	IFileManager& FileManager = IFileManager::Get();
	FDateTime UtilityExecutableModificationTime = FileManager.GetTimeStamp(*UtilityPath);
	FDateTime ServiceExecutableModificationTime = FileManager.GetTimeStamp(*ServicePath);

	FDateTime VersionCacheModificationTime = FileManager.GetTimeStamp(*VersionCachePath);
	bool VersionCacheIsOlderThanUtilityExecutable = VersionCacheModificationTime < UtilityExecutableModificationTime;
	bool VersionCacheIsOlderThanServerExecutable = VersionCacheModificationTime < ServiceExecutableModificationTime;
	bool VersionCacheIsUpToDate = (!VersionCacheIsOlderThanUtilityExecutable) && (!VersionCacheIsOlderThanServerExecutable);
	if (VersionCacheIsUpToDate)
	{
		FString VersionFileContents;
		if (FFileHelper::LoadFileToString(VersionFileContents, *VersionCachePath))
		{
			FZenVersion CachedVersion;
			if (CachedVersion.TryParse(*VersionFileContents))
			{
				UE_LOGF(LogZenServiceInstance, Display, "Read zen version cache file from '%ls', version: '%ls'", *VersionCachePath, *VersionFileContents);
				return CachedVersion;
			}
		}
	}
	auto GetFallbackVersion = [UtilityExecutableModificationTime, ServiceExecutableModificationTime, &UtilityPath]()
		{
			FZenVersion FallbackVersion;
			if (UtilityExecutableModificationTime > ServiceExecutableModificationTime)
			{
				FallbackVersion.Details = UtilityExecutableModificationTime.ToString();
			}
			else
			{
				FallbackVersion.Details = ServiceExecutableModificationTime.ToString();
			}
			FString VersionString = FallbackVersion.ToString();
			UE_LOGF(LogZenServiceInstance, Display, "Using legacy zen version for '%ls', version: '%ls'", *UtilityPath, *VersionString);
			return FallbackVersion;
		};


	FZenVersion Version;
	if (!GetZenVersion(UtilityPath, ServicePath, Version))
	{
		UE_LOGF(LogZenServiceInstance, Warning, "Unable to determine version using zen utility executable path: '%ls'. Using legacy version check.", *UtilityPath);
		Version = GetFallbackVersion();
	}
	FFileHelper::SaveStringToFile(Version.ToString(), *VersionCachePath);
	return Version;
}

static void
PromptUserToSyncInTreeVersion(const FString& ServerFilePath)
{
#if !IS_PROGRAM
	if (!FApp::IsUnattended() && !IsRunningCommandlet() && !GIsRunningUnattendedScript)
	{
		FText ZenSyncSourcePromptTitle = NSLOCTEXT("Zen", "Zen_SyncSourcePromptTitle", "Failed to launch");
		FText ZenSyncSourcePromptText = FText::Format(NSLOCTEXT("Zen", "Zen_SyncSourcePromptText", "Unreal Zen Storage Server can not verify installation. Please make sure your source installation in properly synced at '{0}'"), FText::FromString(FPaths::GetPath(ServerFilePath)));
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *ZenSyncSourcePromptText.ToString(), *ZenSyncSourcePromptTitle.ToString());
	}
	else
#endif
	{
		// Just log as there is no one to show a message
		UE_LOGF(LogZenServiceInstance, Display, "Unreal Zen Storage Server can not verify installation. Please make sure your source installation in properly synced at '%ls'", *FPaths::GetPath(ServerFilePath));
	}
}

static bool
IsInstallVersionOutOfDate(const FString& InTreeUtilityPath, const FString& InstallUtilityPath, const FString& InTreeServicePath, const FString& InstallServicePath, const FString& InTreeVersionCache, const FString& InstallVersionCache, FZenVersion& OutWantedVersion)
{
	IFileManager& FileManager = IFileManager::Get();
	if (!FileManager.FileExists(*InTreeUtilityPath) || !FileManager.FileExists(*InTreeServicePath))
	{
		UE_LOGF(LogZenServiceInstance, Warning, "InTree version at '%ls' is invalid", *InTreeServicePath);
		PromptUserToSyncInTreeVersion(InTreeServicePath);
		return false;
	}

	// Always get the InTree utility path so cached version information is up to date
	FZenVersion InTreeVersion = GetZenVersion(InTreeUtilityPath, InTreeServicePath, InTreeVersionCache);
	UE_LOGF(LogZenServiceInstance, Log, "InTree version at '%ls' is '%ls'", *InTreeServicePath, *InTreeVersion.ToString());

	if (!FileManager.FileExists(*InstallUtilityPath) || !FileManager.FileExists(*InstallServicePath))
	{
		UE_LOGF(LogZenServiceInstance, Log, "No installation found at '%ls'", *InstallServicePath);
		OutWantedVersion = InTreeVersion;
		return true;
	}
	FZenVersion InstallVersion = GetZenVersion(InstallUtilityPath, InstallServicePath, InstallVersionCache);
	UE_LOGF(LogZenServiceInstance, Log, "Installed version at '%ls' is '%ls'", *InstallServicePath, *InstallVersion.ToString());

	if (InstallVersion < InTreeVersion)
	{
		UE_LOGF(LogZenServiceInstance, Log, "Installed version at '%ls' (%ls) is older than '%ls' (%ls)", *InstallServicePath, *InstallVersion.ToString(), *InTreeServicePath, *InTreeVersion.ToString());
		OutWantedVersion = InTreeVersion;
		return true;
	}
	if (FParse::Param(FCommandLine::Get(), TEXT("ForceZenInstall")))
	{
		UE_LOGF(LogZenServiceInstance, Display, "Forcing install from '%ls' (%ls) over '%ls' (%ls)", *InTreeServicePath, *InTreeVersion.ToString(), *InstallServicePath, *InstallVersion.ToString());
		OutWantedVersion = InTreeVersion;
		return true;
	}
	return false;
}

static bool
AttemptFileCopyWithRetries(const TCHAR* Dst, const TCHAR* Src, double RetryDurationSeconds)
{
	IFileManager& FileManager = IFileManager::Get();
	uint32 CopyResult = FileManager.Copy(Dst, Src, true, true, false);
	uint64 CopyWaitStartTime = FPlatformTime::Cycles64();
	while (CopyResult != COPY_OK)
	{
		double CopyWaitDuration = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - CopyWaitStartTime);
		if (CopyWaitDuration < RetryDurationSeconds)
		{
			FPlatformProcess::Sleep(0.01f);
		}
		else
		{
			break;
		}
		CopyResult = FileManager.Copy(Dst, Src, true, true, false);
	}
	if (CopyResult == COPY_OK)
	{
		return true;
	}
	UE_LOGF(LogZenServiceInstance, Warning, "copy from '%ls' to '%ls', '%ls'", Src, Dst, CopyResult == COPY_Fail ? TEXT("Failed to copy file") : TEXT("Cancelled file copy"));
	return false;
}

static bool
AttemptFileDeleteWithRetries(const TCHAR* Path, double RetryDurationSeconds)
{
	IFileManager& FileManager = IFileManager::Get();
	bool DeleteResult = FileManager.Delete(Path, false, false, true);
	uint64 DeleteWaitStartTime = FPlatformTime::Cycles64();
	while (!DeleteResult)
	{
		double DeleteWaitDuration = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - DeleteWaitStartTime);
		if (DeleteWaitDuration < RetryDurationSeconds)
		{
			FPlatformProcess::Sleep(0.01f);
		}
		else
		{
			break;
		}
		DeleteResult = FileManager.Delete(Path, false, false, true);
	}
	if (DeleteResult)
	{
		return true;
	}
	return false;
}

static void EnsureEditorSettingsConfigLoaded()
{
#if !WITH_EDITOR
	if (GEditorSettingsIni.IsEmpty())
	{
		FConfigContext Context = FConfigContext::ReadIntoGConfig();
		Context.GeneratedConfigDir = FPaths::EngineEditorSettingsDir();
		Context.Load(TEXT("EditorSettings"), GEditorSettingsIni);
	}
#endif
}

static void
DetermineLocalDataCachePath(const TCHAR* ConfigSection, FString& DataPath)
{
	FString DataPathEnvOverride;
	if (GConfig->GetString(ConfigSection, TEXT("LocalDataCachePathEnvOverride"), DataPathEnvOverride, GEngineIni))
	{
		FString DataPathEnvOverrideValue = FPlatformMisc::GetEnvironmentVariable(*DataPathEnvOverride);
		if (!DataPathEnvOverrideValue.IsEmpty())
		{
			DataPath = DataPathEnvOverrideValue;
			UE_LOGF(LogZenServiceInstance, Log, "Found environment variable %ls=%ls", *DataPathEnvOverride, *DataPathEnvOverrideValue);
		}

		if (FPlatformMisc::GetStoredValue(TEXT("Epic Games"), TEXT("GlobalDataCachePath"), *DataPathEnvOverride, DataPathEnvOverrideValue))
		{
			if (!DataPathEnvOverrideValue.IsEmpty())
			{
				DataPath = DataPathEnvOverrideValue;
				UE_LOGF(LogZenServiceInstance, Log, "Found registry key GlobalDataCachePath %ls=%ls", *DataPathEnvOverride, *DataPath);
			}
		}
	}

	FString DataPathCommandLineOverride;
	if (GConfig->GetString(ConfigSection, TEXT("LocalDataCachePathCommandLineOverride"), DataPathCommandLineOverride, GEngineIni))
	{
		FString DataPathCommandLineOverrideValue;
		if (FParse::Value(FCommandLine::Get(), *(DataPathCommandLineOverride + TEXT("=")), DataPathCommandLineOverrideValue))
		{
			DataPath = DataPathCommandLineOverrideValue;
			UE_LOGF(LogZenServiceInstance, Log, "Found command line override %ls=%ls", *DataPathCommandLineOverride, *DataPath);
		}
	}

	// Paths starting with a '?' are looked up from config
	if (DataPath.StartsWith(TEXT("?")) && !GConfig->GetString(TEXT("DerivedDataCacheSettings"), *DataPath + 1, DataPath, GEngineIni))
	{
		DataPath.Empty();
	}

	FString DataPathEditorOverrideSetting;
	if (GConfig->GetString(ConfigSection, TEXT("LocalDataCachePathEditorOverrideSetting"), DataPathEditorOverrideSetting, GEngineIni))
	{
		EnsureEditorSettingsConfigLoaded();
		FString Setting = GConfig->GetStr(TEXT("/Script/UnrealEd.EditorSettings"), *DataPathEditorOverrideSetting, GEditorSettingsIni);
		if (!Setting.IsEmpty())
		{
			FString SettingPath;
			if (FParse::Value(*Setting, TEXT("Path="), SettingPath))
			{
				SettingPath.TrimQuotesInline();
				SettingPath.ReplaceEscapedCharWithCharInline();
				if (!SettingPath.IsEmpty())
				{
					DataPath = SettingPath;
					UE_LOGF(LogZenServiceInstance, Log, "Found editor setting /Script/UnrealEd.EditorSettings.Path=%ls", *DataPath);
				}
			}
		}
	}
}

static bool
DetermineDataPath(const TCHAR* ConfigSection, FString& DataPath, bool& bHasInvalidPathConfigurations, bool& bIsDefaultDataPath)
{
	auto ValidateDataPath = [](const FString& InDataPath)
	{
		if (InDataPath.IsEmpty())
		{
			return FString{};
		}
		IFileManager& FileManager = IFileManager::Get();
		FString FinalPath = FPaths::ConvertRelativePathToFull(InDataPath);
		FPaths::NormalizeDirectoryName(FinalPath);
		FFileStatData StatData = FileManager.GetStatData(*InDataPath);
		if (StatData.bIsValid && StatData.bIsDirectory)
		{
			FString TestFilePath = FinalPath / FString::Printf(TEXT(".zen-startup-test-file-%d"), FPlatformProcess::GetCurrentProcessId());
			FArchive* TestFile = FileManager.CreateFileWriter(*TestFilePath, FILEWRITE_Silent);
			if (!TestFile)
			{
				return FString{};
			}
			TestFile->Close();
			delete TestFile;
			FileManager.Delete(*TestFilePath);
			return FinalPath;
		}
		if (FileManager.MakeDirectory(*InDataPath, true))
		{
			return FinalPath;
		}
		return FString{};
	};

	// Zen commandline
	FString CommandLineOverrideValue;
	if (FParse::Value(FCommandLine::Get(), TEXT("ZenDataPath="), CommandLineOverrideValue) && !CommandLineOverrideValue.IsEmpty())
	{
		if (FString Path = ValidateDataPath(CommandLineOverrideValue); !Path.IsEmpty())
		{
			DataPath = Path;
			UE_LOGF(LogZenServiceInstance, Log, "Found command line override ZenDataPath=%ls", *CommandLineOverrideValue);
			return true;
		}
		UE_LOGF(LogZenServiceInstance, Warning, "Skipping command line override ZenDataPath=%ls due to an invalid path", *CommandLineOverrideValue);
		bHasInvalidPathConfigurations = true;
	}

	// Zen subprocess environment
	if (FString SubprocessDataPathEnvOverrideValue = FPlatformMisc::GetEnvironmentVariable(TEXT("UE-ZenSubprocessDataPath")); !SubprocessDataPathEnvOverrideValue.IsEmpty())
	{
		if (FString Path = ValidateDataPath(SubprocessDataPathEnvOverrideValue); !Path.IsEmpty())
		{
			DataPath = Path;
			UE_LOGF(LogZenServiceInstance, Log, "Found subprocess environment variable UE-ZenSubprocessDataPath=%ls", *SubprocessDataPathEnvOverrideValue);
			return true;
		}
		UE_LOGF(LogZenServiceInstance, Warning, "Skipping subprocess environment variable UE-ZenSubprocessDataPath=%ls due to an invalid path", *SubprocessDataPathEnvOverrideValue);
		bHasInvalidPathConfigurations = true;
	}

	// Zen registry/stored
	FString DataPathEnvOverrideValue;
	if (FPlatformMisc::GetStoredValue(TEXT("Epic Games"), TEXT("Zen"), TEXT("DataPath"), DataPathEnvOverrideValue) && !DataPathEnvOverrideValue.IsEmpty())
	{
		if (FString Path = ValidateDataPath(DataPathEnvOverrideValue); !Path.IsEmpty())
		{
			DataPath = Path;
			UE_LOGF(LogZenServiceInstance, Log, "Found registry key Zen DataPath=%ls", *DataPathEnvOverrideValue);
			return true;
		}
		UE_LOGF(LogZenServiceInstance, Warning, "Skipping registry key Zen DataPath=%ls due to an invalid path", *DataPathEnvOverrideValue);
		bHasInvalidPathConfigurations = true;
	}

	// Zen environment
	if (FString ZenDataPathEnvOverrideValue = FPlatformMisc::GetEnvironmentVariable(TEXT("UE-ZenDataPath")); !ZenDataPathEnvOverrideValue.IsEmpty())
	{
		if (FString Path = ValidateDataPath(ZenDataPathEnvOverrideValue); !Path.IsEmpty())
		{
			DataPath = Path;
			UE_LOGF(LogZenServiceInstance, Log, "Found environment variable UE-ZenDataPath=%ls", *ZenDataPathEnvOverrideValue);
			return true;
		}
		UE_LOGF(LogZenServiceInstance, Warning, "Skipping environment variable UE-ZenDataPath=%ls due to an invalid path", *ZenDataPathEnvOverrideValue);
		bHasInvalidPathConfigurations = true;
	}

	// Follow local DDC (if outside workspace)
	FString LocalDataCachePath;
	DetermineLocalDataCachePath(ConfigSection, LocalDataCachePath);
	if (!LocalDataCachePath.IsEmpty() && (LocalDataCachePath != TEXT("None")) && !FPaths::IsUnderDirectory(LocalDataCachePath, FPaths::RootDir()))
	{
		FString ZenLocalDataCachePath = FPaths::Combine(LocalDataCachePath, TEXT("Zen"));
		if (FString Path = ValidateDataPath(ZenLocalDataCachePath); !Path.IsEmpty())
		{
			DataPath = Path;
			UE_LOGF(LogZenServiceInstance, Log, "Found local data cache path=%ls", *LocalDataCachePath);
			return true;
		}
		UE_LOGF(LogZenServiceInstance, Warning, "Skipping local data cache path=%ls due to an invalid path", *LocalDataCachePath);
		bHasInvalidPathConfigurations = true;
	}

	// Zen config default
	FString ConfigDefaultPath;
	GConfig->GetString(ConfigSection, TEXT("DataPath"), ConfigDefaultPath, GEngineIni);
	if (!ConfigDefaultPath.IsEmpty())
	{
		ConfigDefaultPath.ReplaceInline(TEXT("%ENGINEVERSIONAGNOSTICINSTALLEDUSERDIR%"), *GetLocalZenRootPath());
		if (FString Path = ValidateDataPath(ConfigDefaultPath); !Path.IsEmpty())
		{
			DataPath = Path;
			bIsDefaultDataPath = true;
			UE_LOGF(LogZenServiceInstance, Log, "Found Zen config default=%ls", *ConfigDefaultPath);
			return true;
		}
		UE_LOGF(LogZenServiceInstance, Warning, "Skipping Zen config default=%ls due to an invalid path", *ConfigDefaultPath);
		bHasInvalidPathConfigurations = true;
	}
	UE_LOGF(LogZenServiceInstance, Warning, "Unable to determine a valid Zen data path");
	return false;
}

static void
ReadUInt16FromConfig(const TCHAR* Section, const TCHAR* Key, uint16& Value, const FString& ConfigFile)
{
	int32 ValueInt32 = Value;
	GConfig->GetInt(Section, Key, ValueInt32, ConfigFile);
	Value = (uint16)ValueInt32;
}

static bool
IsLocalHost(const FString& Host)
{
	if (Host.Compare(FString(TEXT("localhost")), ESearchCase::IgnoreCase) == 0)
	{
		return true;
	}

	if (Host.Compare(FString(TEXT("127.0.0.1"))) == 0)
	{
		return true;
	}

	if (Host.Compare(FString(TEXT("[::1]"))) == 0)
	{
		return true;
	}

	return false;
}

static void
ApplyProcessLifetimeOverride(bool& bLimitProcessLifetime)
{
	FString LimitProcessLifetime = FPlatformMisc::GetEnvironmentVariable(TEXT("UE-ZenLimitProcessLifetime"));
	if (!LimitProcessLifetime.IsEmpty())
	{
		bLimitProcessLifetime = LimitProcessLifetime.ToBool();
	}
}

static void
PromptUserUnableToDetermineValidDataPath()
{
#if !IS_PROGRAM
	if (!FApp::IsUnattended() && !IsRunningCommandlet() && !GIsRunningUnattendedScript)
	{
		FString LogDirPath = FPaths::ConvertRelativePathToFull(FPaths::ProjectLogDir());
		FText ZenInvalidDataPathPromptTitle = NSLOCTEXT("Zen", "Zen_InvalidDataPathPromptTitle", "No Valid Data Path Configuration");
		FText ZenInvalidDataPathPromptText = FText::Format(NSLOCTEXT("Zen", "Zen_InvalidDataPathPromptText", "Unreal Zen Storage Server can not determine a valid data path.\nPlease check the log in '{0}' for details.\nUpdate your configuration and restart."), FText::FromString(LogDirPath));
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *ZenInvalidDataPathPromptText.ToString(), *ZenInvalidDataPathPromptTitle.ToString());
	}
	else
#endif
	{
		// Just log as there is no one to show a message
		UE_LOGF(LogZenServiceInstance, Warning, "Unreal Zen Storage Server is unable to determine a valid data path");
	}
}

static void
PromptUserInvalidSystemServiceDataPath()
{
#if !IS_PROGRAM
	if (!FApp::IsUnattended() && !IsRunningCommandlet() && !GIsRunningUnattendedScript)
	{
		FString LogDirPath = FPaths::ConvertRelativePathToFull(FPaths::ProjectLogDir());
		FText ZenInvalidDataPathPromptTitle = NSLOCTEXT("Zen", "Zen_InvalidDataPathPromptTitle", "No Valid Data Path Configuration");
		FText ZenInvalidDataPathPromptText = FText::Format(NSLOCTEXT("Zen", "Zen_InvalidSystemServiceDataPathPromptText", "Unreal Zen Storage Server can not determine a valid data path.\nPlease check the log in '{0}' for details.\nFalling back to Zen AutoLaunch or ConnectExisting settings."), FText::FromString(LogDirPath));
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *ZenInvalidDataPathPromptText.ToString(), *ZenInvalidDataPathPromptTitle.ToString());
	}
	else
#endif
	{
		// Just log as there is no one to show a message
		UE_LOGF(LogZenServiceInstance, Warning, "Unreal Zen Storage Server is unable to determine a valid data path for the system service.");
	}
}

static void
PromptUserAboutInvalidValidDataPathConfiguration(const FString& UsedDataPath)
{
#if !IS_PROGRAM
	if (!FApp::IsUnattended() && !IsRunningCommandlet() && !GIsRunningUnattendedScript)
	{
		FString LogDirPath = FPaths::ConvertRelativePathToFull(FPaths::ProjectLogDir());
		FText ZenInvalidValidDataPathConfigurationPromptTitle = NSLOCTEXT("Zen", "Zen_InvalidValidDataPathConfigurationPromptTitle", "Invalid Data Paths");
		FText ZenInvalidValidDataPathConfigurationPromptText = FText::Format(NSLOCTEXT("Zen", "Zen_InvalidValidDataPathConfigurationPromptText", "Unreal Zen Storage Server has detected invalid data path configuration.\nPlease check the log in '{0}' for details.\n\nFalling back to using '{1}' as data path."), FText::FromString(LogDirPath), FText::FromString(UsedDataPath));
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *ZenInvalidValidDataPathConfigurationPromptText.ToString(), *ZenInvalidValidDataPathConfigurationPromptTitle.ToString());
	}
	else
#endif
	{
		// Just log as there is no one to show a message
		UE_LOGF(LogZenServiceInstance, Warning, "Unreal Zen Storage Server has detected invalid data path configuration. Falling back to '%ls'", *UsedDataPath);
	}
}

#if PLATFORM_WINDOWS
static void
PromptUserIsUsingGoogleDriveAsDataPath(const FString& BadDataPath)
{
#if !IS_PROGRAM
	if (!FApp::IsUnattended() && !IsRunningCommandlet() && !GIsRunningUnattendedScript)
	{
		FString LogDirPath = FPaths::ConvertRelativePathToFull(FPaths::ProjectLogDir());
		FText ZenInvalidDataPathPromptTitle = NSLOCTEXT("Zen", "Zen_GoogleDriveDataPathPromptTitle", "Using Google Drive as a data path");
		FText ZenInvalidDataPathPromptText = FText::Format(NSLOCTEXT("Zen", "Zen_GoogleDriveDataPathPromptText", "Unreal Zen Storage Server is configured to use Google Drive '{0}' as a data path, this is highly inadvisable.\nPlease use a data path on a local physical drive.\nCheck the log in '{1}' for details.\nUpdate your configuration and restart."), FText::FromString(BadDataPath), FText::FromString(LogDirPath));
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *ZenInvalidDataPathPromptText.ToString(), *ZenInvalidDataPathPromptTitle.ToString());
	}
	else
#endif
	{
		// Just log as there is no one to show a message
		UE_LOGF(LogZenServiceInstance, Warning, "Unreal Zen Storage Server is configured to use Google Drive as a data path '%ls', this is highly inadvisable. Please use a path on a local physical drive.", *BadDataPath);
	}
}

static void
PromptUserIsUsingNetworkDriveAsDataPath(const FString& BadDataPath)
{
#if !IS_PROGRAM
	if (!FApp::IsUnattended() && !IsRunningCommandlet() && !GIsRunningUnattendedScript)
	{
		FString LogDirPath = FPaths::ConvertRelativePathToFull(FPaths::ProjectLogDir());
		FText ZenInvalidDataPathPromptTitle = NSLOCTEXT("Zen", "Zen_NetworkDriveDataPathPromptTitle", "Using Network Drive as a data path");
		FText ZenInvalidDataPathPromptText = FText::Format(NSLOCTEXT("Zen", "Zen_NetworkDriveDataPathPromptText", "Unreal Zen Storage Server is configured to a network path '{0}' as a data path, this is highly inadvisable.\nPlease use a data path on a local physical drive.\nCheck the log in '{1}' for details.\nUpdate your configuration and restart."), FText::FromString(BadDataPath), FText::FromString(LogDirPath));
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *ZenInvalidDataPathPromptText.ToString(), *ZenInvalidDataPathPromptTitle.ToString());
	}
	else
#endif
	{
		// Just log as there is no one to show a message
		UE_LOGF(LogZenServiceInstance, Warning, "Unreal Zen Storage Server is configured to use Network Drive as a data path '%ls', this is highly inadvisable. Please use a path on a local physical drive.", *BadDataPath);
	}
}
#endif // PLATFORM_WINDOWS

static void ReadCbField(FCbFieldView Field, UE::Zen::FZenSizeStats& OutValue)
{
	FCbObjectView ObjectView = Field.AsObjectView();
	OutValue.Disk = ObjectView["disk"].AsDouble();
	OutValue.Memory = ObjectView["memory"].AsDouble();
}

static void ReadCbField(FCbFieldView Field, UE::Zen::FZenCIDSizeStats& OutValue)
{
	FCbObjectView ObjectView = Field.AsObjectView();
	OutValue.Tiny = ObjectView["tiny"].AsInt64();
	OutValue.Small = ObjectView["small"].AsInt64();
	OutValue.Large = ObjectView["large"].AsInt64();
	OutValue.Total = ObjectView["total"].AsInt64();
}

static void ReadCbField(FCbFieldView Field, UE::Zen::FZenCIDStats& OutValue)
{
	FCbObjectView ObjectView = Field.AsObjectView();
	ReadCbField(ObjectView["size"], OutValue.Size);
}

static FServiceAutoLaunchSettings::EInstallMode ZenGetInstallModeFromString(const FString& InstallMode)
{
	if (InstallMode.IsEmpty() || FCString::Stricmp(*InstallMode, TEXT("auto")) == 0)
	{
		return FApp::IsEngineInstalled() ? FServiceAutoLaunchSettings::EInstallMode::Link : FServiceAutoLaunchSettings::EInstallMode::Copy;
	}
	else if (FCString::Stricmp(*InstallMode, TEXT("copy")) == 0)
	{
		return FServiceAutoLaunchSettings::EInstallMode::Copy;
	}
	else if (FCString::Stricmp(*InstallMode, TEXT("link")) == 0)
	{
		return FServiceAutoLaunchSettings::EInstallMode::Link;
	}
	else
	{
		checkf(false, TEXT("Invalid zenserver install mode: {%s}"), *InstallMode);
		return FServiceAutoLaunchSettings::EInstallMode::Copy;
	}
}

static FString ZenGetInstallModeToString(FServiceAutoLaunchSettings::EInstallMode InstallMode)
{
	switch (InstallMode)
	{
	case FServiceAutoLaunchSettings::EInstallMode::Copy:
		return TEXT("copy");
	case FServiceAutoLaunchSettings::EInstallMode::Link:
		return TEXT("link");
	}
	checkf(false, TEXT("Invalid zenserver install mode: {%d}"), static_cast<int>(InstallMode));
	return "";
}

static ERemoteNetworkService ZenGetRemoteNetworkServiceFromString(const FString& Value)
{
	if (Value.IsEmpty() || FCString::Stricmp(*Value, TEXT("None")) == 0)
	{
		return ERemoteNetworkService::None;
	}
	else if (FCString::Stricmp(*Value, TEXT("Unsecured")) == 0)
	{
		return ERemoteNetworkService::Unsecured;
	}
	else if (FCString::Stricmp(*Value, TEXT("GeneratedStaticKey")) == 0)
	{
		return ERemoteNetworkService::GeneratedStaticKey;
	}
	else
	{
		checkf(false, TEXT("Unrecognized remote network service type: {%s}"), *Value);
		return ERemoteNetworkService::None;
	}
}

static FString ZenGetRemoteNetworkServiceToString(ERemoteNetworkService Value)
{
	switch (Value)
	{
	case ERemoteNetworkService::None:          return TEXT("None");
	case ERemoteNetworkService::Unsecured:     return TEXT("Unsecured");
	case ERemoteNetworkService::GeneratedStaticKey: return TEXT("GeneratedStaticKey");
	}
	checkf(false, TEXT("Unrecognized remote network service type: %d"), static_cast<int>(Value));
	return TEXT("");
}

bool FServicePluginSettings::ReadFromConfig(const FString& InPluginName)
{
	const FString PluginSectionName = FString::Format(TEXT("Zen.Plugin.{0}"), {*InPluginName}); 
	const FConfigSection* PluginSection = GConfig->GetSection(*PluginSectionName, false, GEngineIni);
	if (!PluginSection)
	{
		UE_LOGF(LogZenServiceInstance, Warning, "Unable to find config section '%ls'", *PluginSectionName);
		return false;
	}

	bool bHasName = false;
	for (FConfigSection::TConstIterator It(*PluginSection); It; ++It)
	{
		if (It.Key() == TEXT("Name"))
		{
			Name = It.Value().GetValue();
			// TODO load plugins from project dir
			FString AbsPathRoot = FPaths::ConvertRelativePathToFull(FPaths::Combine(*FPaths::RootDir(), *Name));
			if (IFileManager::Get().FileExists(*AbsPathRoot))
			{
				AbsPath = AbsPathRoot;
			}
			else
			{
				UE_LOGF(LogZenServiceInstance, Display, "Can't find file for plugin '%ls', tried '%ls'", *Name, *AbsPathRoot);
				return false;
			}

			Name = FPaths::GetBaseFilename(Name);
			bHasName = true;
		}
		else
		{
			Options.Add(It.Key(), It.Value().GetValue());
		}
	}

	if (!bHasName)
	{
		UE_LOGF(LogZenServiceInstance, Warning, "Settings for plugin defined in section '%ls' are missing 'Name' value", *PluginSectionName);
	}

	return bHasName;
}

bool FServicePluginSettings::ReadFromCompactBinary(FCbFieldView Field)
{
	if (!Field.IsObject())
	{
		return false;
	}

	bool bValid = true;

	Name = FString(Field["Name"].AsString());
	bValid &= !Name.IsEmpty();

	AbsPath = FString(Field["AbsPath"].AsString());
	bValid &= !AbsPath.IsEmpty();

	if (FCbObjectView OptionsObject = Field["Options"].AsObjectView())
	{
		for (FCbFieldViewIterator It = OptionsObject.CreateViewIterator(); It;)
		{
			if (!It->IsString())
			{
				bValid = false;
				break;
			}
			FString OptionName = FString(It->AsString());
			++It;
			if (!It->IsString())
			{
				bValid = false;
				break;
			}
			FString OptionValue = FString(It->AsString());
			Options.Add(FName(OptionName), OptionValue);
		}
	}

	return bValid;
}

void FServicePluginSettings::WriteToCompactBinary(FCbWriter& Writer) const
{
	Writer.BeginObject();
	Writer << "Name" << Name;
	Writer << "AbsPath" << AbsPath;
	if (!Options.IsEmpty())
	{
		Writer.BeginObject("Options");
		for (const TPair<FName, FString>& Option: Options)
		{
			Writer << Option.Key << Option.Value;
		}
		Writer.EndObject();
	}
	Writer.EndObject();
}

static TOptional<FSystemServiceSettings> ReadSystemServiceSettingsFromConfig()
{
	const TCHAR* SystemServiceConfigSection = TEXT("Zen.SystemService");
	FSystemServiceSettings SystemServiceSettings;
	bool bHasInvalidPathConfigurations = false;
	bool bIsDefaultDataPath = false;

	if (!DetermineDataPath(SystemServiceConfigSection, SystemServiceSettings.DataPath, bHasInvalidPathConfigurations, bIsDefaultDataPath))
	{
		PromptUserInvalidSystemServiceDataPath();
		return {};
	}
	else if (bHasInvalidPathConfigurations)
	{
		PromptUserAboutInvalidValidDataPathConfiguration(SystemServiceSettings.DataPath);
	}

	GConfig->GetString(SystemServiceConfigSection, TEXT("InstallPath"), SystemServiceSettings.InstallPath, GEngineIni);
	GConfig->GetString(SystemServiceConfigSection, TEXT("ExtraArgs"), SystemServiceSettings.ExtraArgs, GEngineIni);
	FString RemoteNetworkServiceStr;
	if (GConfig->GetString(SystemServiceConfigSection, TEXT("RemoteNetworkService"), RemoteNetworkServiceStr, GEngineIni))
	{
		SystemServiceSettings.RemoteNetworkService = ZenGetRemoteNetworkServiceFromString(RemoteNetworkServiceStr);
	}
	EnsureEditorSettingsConfigLoaded();
	GConfig->GetBool(TEXT("/Script/UnrealEd.CrashReportsPrivacySettings"), TEXT("bSendUnattendedBugReports"), SystemServiceSettings.bSendUnattendedBugReports, GEditorSettingsIni);

	return TOptional<FSystemServiceSettings>(SystemServiceSettings);
}

bool
FServiceSettings::ReadFromConfig()
{
	check(GConfig && GConfig->IsReadyForUse());
	const TCHAR* ConfigSection = TEXT("Zen");
	bool bAutoLaunch = true;
	GConfig->GetBool(ConfigSection, TEXT("AutoLaunch"), bAutoLaunch, GEngineIni);
	GConfig->GetBool(ConfigSection, TEXT("SystemService"), bSystemService, GEngineIni);

	if (bSystemService)
	{
		TOptional<FSystemServiceSettings> Settings = ReadSystemServiceSettingsFromConfig();
		if (Settings)
		{
			bSystemService = true;
			SystemServiceSettings = *Settings;
		}
	}

	if (bAutoLaunch)
	{
		if (!TryApplyAutoLaunchOverride())
		{
			// AutoLaunch settings
			const TCHAR* AutoLaunchConfigSection = TEXT("Zen.AutoLaunch");
			SettingsVariant.Emplace<FServiceAutoLaunchSettings>();
			FServiceAutoLaunchSettings& AutoLaunchSettings = SettingsVariant.Get<FServiceAutoLaunchSettings>();

			bool bHasInvalidPathConfigurations = false;
			if (!DetermineDataPath(AutoLaunchConfigSection, AutoLaunchSettings.DataPath, bHasInvalidPathConfigurations, AutoLaunchSettings.bIsDefaultDataPath))
			{
				PromptUserUnableToDetermineValidDataPath();
				return false;
			}
			else if (bHasInvalidPathConfigurations)
			{
				PromptUserAboutInvalidValidDataPathConfiguration(AutoLaunchSettings.DataPath);
			}

#if PLATFORM_WINDOWS
			{
				int32 DriveEnd = 0;
				if (AutoLaunchSettings.DataPath.FindChar(':', DriveEnd))
				{
					FString DrivePath = AutoLaunchSettings.DataPath.Left(DriveEnd + 2);

					UINT DriveType = GetDriveType(*DrivePath);
					if (DriveType == DRIVE_REMOTE)
					{
						PromptUserIsUsingNetworkDriveAsDataPath(AutoLaunchSettings.DataPath);
					}
					else
					{
						TCHAR VolumeName[128];

						BOOL OK = GetVolumeInformation(
							*DrivePath,
							VolumeName,
							127,
							NULL,
							NULL,
							NULL,
							NULL,
							NULL);

						if (OK)
						{
							VolumeName[127] = 0;
							if (FString(VolumeName) == TEXT("Google Drive"))
							{
								PromptUserIsUsingGoogleDriveAsDataPath(AutoLaunchSettings.DataPath);
							}
						}
					}
				}
				else if (AutoLaunchSettings.DataPath.StartsWith("\\\\") || AutoLaunchSettings.DataPath.StartsWith("//"))
				{
					PromptUserIsUsingNetworkDriveAsDataPath(AutoLaunchSettings.DataPath);
				}
			}
#endif // PLATFORM_WINDOWS

			GConfig->GetString(AutoLaunchConfigSection, TEXT("ExtraArgs"), AutoLaunchSettings.ExtraArgs, GEngineIni);

			ReadUInt16FromConfig(AutoLaunchConfigSection, TEXT("DesiredPort"), AutoLaunchSettings.DesiredPort, GEngineIni);
			AutoLaunchSettings.Plugins.Empty();
			TArray<FString> PluginNames;
			GConfig->GetArray(AutoLaunchConfigSection, TEXT("Plugins"), PluginNames, GEngineIni);
			for (FString PluginName : PluginNames)
			{
				FServicePluginSettings PluginSettings = {};
				if (PluginSettings.ReadFromConfig(PluginName))
				{
					AutoLaunchSettings.Plugins.Add(PluginSettings);
				}
			}
			GConfig->GetBool(AutoLaunchConfigSection, TEXT("ShowConsole"), AutoLaunchSettings.bShowConsole, GEngineIni);
			GConfig->GetBool(AutoLaunchConfigSection, TEXT("LimitProcessLifetime"), AutoLaunchSettings.bLimitProcessLifetime, GEngineIni);
			ApplyProcessLifetimeOverride(AutoLaunchSettings.bLimitProcessLifetime);
			FString RemoteNetworkServiceStr;
			if (GConfig->GetString(AutoLaunchConfigSection, TEXT("RemoteNetworkService"), RemoteNetworkServiceStr, GEngineIni))
			{
				AutoLaunchSettings.RemoteNetworkService = ZenGetRemoteNetworkServiceFromString(RemoteNetworkServiceStr);
			}
			FString InstallMode;
			if (GConfig->GetString(AutoLaunchConfigSection, TEXT("InstallMode"), InstallMode, GEngineIni))
			{
				AutoLaunchSettings.InstallMode = ZenGetInstallModeFromString(InstallMode);
			}
			EnsureEditorSettingsConfigLoaded();
			GConfig->GetBool(TEXT("/Script/UnrealEd.CrashReportsPrivacySettings"), TEXT("bSendUnattendedBugReports"), AutoLaunchSettings.bSendUnattendedBugReports, GEditorSettingsIni);
		}
	}
	else
	{
		// ConnectExisting settings
		const TCHAR* ConnectExistingConfigSection = TEXT("Zen.ConnectExisting");
		SettingsVariant.Emplace<FServiceConnectSettings>();
		FServiceConnectSettings& ConnectExistingSettings = SettingsVariant.Get<FServiceConnectSettings>();

		GConfig->GetString(ConnectExistingConfigSection, TEXT("HostName"), ConnectExistingSettings.HostName, GEngineIni);
		ReadUInt16FromConfig(ConnectExistingConfigSection, TEXT("Port"), ConnectExistingSettings.Port, GEngineIni);
	}
	return true;
}

bool
FServiceSettings::ReadFromCompactBinary(FCbFieldView Field)
{
	if (Field.IsObject())
	{
		if (bool bSystemServiceValue = Field["bSystemService"].AsBool())
		{
			bSystemService = true;

			if (FCbObjectView SystemSettingsObject = Field["SystemServiceSettings"].AsObjectView())
			{
				SystemServiceSettings.DataPath = FString(SystemSettingsObject["DataPath"].AsString());
				SystemServiceSettings.ExtraArgs = FString(SystemSettingsObject["ExtraArgs"].AsString());
				SystemServiceSettings.DesiredPort = SystemSettingsObject["DesiredPort"].AsInt16();
				SystemServiceSettings.InstallPath = FString(SystemSettingsObject["InstallPath"].AsString());
				SystemServiceSettings.RemoteNetworkService = ZenGetRemoteNetworkServiceFromString(FString(SystemSettingsObject["RemoteNetworkService"].AsString()));
				SystemServiceSettings.bSendUnattendedBugReports = SystemSettingsObject["SendUnattendedBugReports"].AsBool();
			}
		}

		if (bool bAutoLaunchValue = Field["bAutoLaunch"].AsBool())
		{
			if (!TryApplyAutoLaunchOverride())
			{
				SettingsVariant.Emplace<FServiceAutoLaunchSettings>();
				FServiceAutoLaunchSettings& AutoLaunchSettings = SettingsVariant.Get<FServiceAutoLaunchSettings>();

				if (FCbObjectView AutoLaunchSettingsObject = Field["AutoLaunchSettings"].AsObjectView())
				{
					AutoLaunchSettings.DataPath = FString(AutoLaunchSettingsObject["DataPath"].AsString());
					AutoLaunchSettings.ExtraArgs = FString(AutoLaunchSettingsObject["ExtraArgs"].AsString());
					AutoLaunchSettings.DesiredPort = AutoLaunchSettingsObject["DesiredPort"].AsInt16();
					if (FCbArrayView PluginsView = AutoLaunchSettingsObject["Plugins"].AsArrayView())
					{
						AutoLaunchSettings.Plugins.Empty();
						for (FCbFieldView& PluginView : PluginsView)
						{
							FServicePluginSettings PluginSettings = {};
							if (PluginSettings.ReadFromCompactBinary(PluginView))
							{
								AutoLaunchSettings.Plugins.Add(PluginSettings);
							}
						}
					}
					AutoLaunchSettings.bShowConsole = AutoLaunchSettingsObject["ShowConsole"].AsBool();
					AutoLaunchSettings.bIsDefaultDataPath = AutoLaunchSettingsObject["IsDefaultDataPath"].AsBool();
					AutoLaunchSettings.bLimitProcessLifetime = AutoLaunchSettingsObject["LimitProcessLifetime"].AsBool();
					ApplyProcessLifetimeOverride(AutoLaunchSettings.bLimitProcessLifetime);
					AutoLaunchSettings.RemoteNetworkService = ZenGetRemoteNetworkServiceFromString(FString(AutoLaunchSettingsObject["RemoteNetworkService"].AsString()));
					AutoLaunchSettings.bSendUnattendedBugReports = AutoLaunchSettingsObject["SendUnattendedBugReports"].AsBool();
					AutoLaunchSettings.bIsDefaultSharedRunContext = AutoLaunchSettingsObject["IsDefaultSharedRunContext"].AsBool(AutoLaunchSettings.bIsDefaultSharedRunContext);
					AutoLaunchSettings.InstallMode = ZenGetInstallModeFromString(FString(AutoLaunchSettingsObject["InstallMode"].AsString()));
				}
			}
		}
		else
		{
			SettingsVariant.Emplace<FServiceConnectSettings>();
			FServiceConnectSettings& ConnectExistingSettings = SettingsVariant.Get<FServiceConnectSettings>();

			if (FCbObjectView ConnectExistingSettingsObject = Field["ConnectExistingSettings"].AsObjectView())
			{
				ConnectExistingSettings.HostName = FString(ConnectExistingSettingsObject["HostName"].AsString());
				ConnectExistingSettings.Port = ConnectExistingSettingsObject["Port"].AsInt16();
			}
		}
		return true;
	}
	return false;
}

bool
FServiceSettings::ReadFromURL(FStringView InstanceURL)
{
	SettingsVariant.Emplace<FServiceConnectSettings>();
	FServiceConnectSettings& ConnectExistingSettings = SettingsVariant.Get<FServiceConnectSettings>();

	if (InstanceURL.StartsWith(TEXT("http://")))
	{
		InstanceURL.RightChopInline(7);
	}

	// Is the given URL perhaps a Unix domain socket path?
	if (InstanceURL.EndsWith(TEXT(".sock")))
	{
		ConnectExistingSettings.HostName = InstanceURL;
		ConnectExistingSettings.Port = 0;
		return true;
	}

	int32 PortDelimIndex = INDEX_NONE;
	InstanceURL.FindLastChar(TEXT(':'), PortDelimIndex);
	if (PortDelimIndex != INDEX_NONE)
	{
		ConnectExistingSettings.HostName = InstanceURL.Left(PortDelimIndex);
		LexFromString(ConnectExistingSettings.Port, InstanceURL.RightChop(PortDelimIndex + 1));
	}
	else
	{
		ConnectExistingSettings.HostName = InstanceURL;
		ConnectExistingSettings.Port = 8558;
	}
	return true;
}

void
FServiceSettings::WriteToCompactBinary(FCbWriter& Writer) const
{
	bool bAutoLaunch = IsAutoLaunch();
	Writer << "bAutoLaunch" << bAutoLaunch;
	Writer << "bSystemService" << IsSystemService();

	if (bSystemService)
	{
		Writer.BeginObject("SystemServiceSettings");
		Writer << "DataPath" << SystemServiceSettings.DataPath;
		Writer << "ExtraArgs" << SystemServiceSettings.ExtraArgs;
		Writer << "DesiredPort" << SystemServiceSettings.DesiredPort;
		Writer << "InstallPath" << SystemServiceSettings.InstallPath;
		Writer << "SendUnattendedBugReports" << SystemServiceSettings.bSendUnattendedBugReports;
		Writer << "RemoteNetworkService" << ZenGetRemoteNetworkServiceToString(SystemServiceSettings.RemoteNetworkService);
		Writer.EndObject();
	}

	if (bAutoLaunch)
	{
		const FServiceAutoLaunchSettings& AutoLaunchSettings = SettingsVariant.Get<FServiceAutoLaunchSettings>();
		Writer.BeginObject("AutoLaunchSettings");
		Writer << "DataPath" << AutoLaunchSettings.DataPath;
		Writer << "ExtraArgs" <<AutoLaunchSettings.ExtraArgs;
		Writer << "DesiredPort" << AutoLaunchSettings.DesiredPort;

		if (!AutoLaunchSettings.Plugins.IsEmpty())
		{
			Writer.BeginArray("Plugins");
			for (const FServicePluginSettings& PluginSettings : AutoLaunchSettings.Plugins)
			{
				PluginSettings.WriteToCompactBinary(Writer);
			}
			Writer.EndArray();
		}
		Writer << "ShowConsole" << AutoLaunchSettings.bShowConsole;
		Writer << "IsDefaultDataPath" << AutoLaunchSettings.bIsDefaultDataPath;
		Writer << "LimitProcessLifetime" << AutoLaunchSettings.bLimitProcessLifetime;
		Writer << "RemoteNetworkService" << ZenGetRemoteNetworkServiceToString(AutoLaunchSettings.RemoteNetworkService);
		Writer << "SendUnattendedBugReports" << AutoLaunchSettings.bSendUnattendedBugReports;
		Writer << "IsDefaultSharedRunContext" << AutoLaunchSettings.bIsDefaultSharedRunContext;
		Writer << "InstallMode" << ZenGetInstallModeToString(AutoLaunchSettings.InstallMode);
		Writer.EndObject();
	}
	else
	{
		const FServiceConnectSettings& ConnectExistingSettings = SettingsVariant.Get<FServiceConnectSettings>();
		Writer.BeginObject("ConnectExistingSettings");
		Writer << "HostName" << ConnectExistingSettings.HostName;
		Writer << "Port" << ConnectExistingSettings.Port;
		Writer.EndObject();
	}
}

bool
FServiceSettings::TryApplyAutoLaunchOverride()
{
#if ALLOW_SETTINGS_OVERRIDE_FROM_COMMANDLINE
	if (FParse::Param(FCommandLine::Get(), TEXT("NoZenAutoLaunch")))
	{
		SettingsVariant.Emplace<FServiceConnectSettings>();
		FServiceConnectSettings& ConnectExistingSettings = SettingsVariant.Get<FServiceConnectSettings>();
		ConnectExistingSettings.HostName = TEXT("[::1]");
		ConnectExistingSettings.Port = 8558;
		return true;
	}

	FString Host;
	if  (FParse::Value(FCommandLine::Get(), TEXT("-NoZenAutoLaunch="), Host))
	{
		SettingsVariant.Emplace<FServiceConnectSettings>();
		FServiceConnectSettings& ConnectExistingSettings = SettingsVariant.Get<FServiceConnectSettings>();

		int32 PortDelimIndex = INDEX_NONE;
		if (Host.FindChar(TEXT(':'), PortDelimIndex))
		{
			ConnectExistingSettings.HostName = Host.Left(PortDelimIndex);
			LexFromString(ConnectExistingSettings.Port, Host.RightChop(PortDelimIndex + 1));
		}
		else
		{
			ConnectExistingSettings.HostName = Host;
			ConnectExistingSettings.Port = 8558;
		}

		return true;
	}
#endif
	return false;
}

uint16 FZenServiceInstance::AutoLaunchedPort = 0;
uint32 FZenServiceInstance::AutoLaunchedPid = 0;
const uint32 FZenServiceInstance::DefaultSmallOperationConnectTimeoutMS = 500; // 0.5 second connection timeout
const uint32 FZenServiceInstance::DefaultSmallOperationTimeoutMS = 60000; // 1 minute total duration timout

static bool
IsZenProcessUsingEffectivePort(uint16 EffectiveListenPort)
{
	ZenSharedEvent ShutDownEvent(ZenSharedEvent::GetShutdownEventName(EffectiveListenPort));
	return ShutDownEvent.Exists();
}

static bool
RequestZenShutdownOnEffectivePort(uint16 EffectiveListenPort)
{
	ZenSharedEvent ShutDownEvent(ZenSharedEvent::GetShutdownEventName(EffectiveListenPort));
	if (!ShutDownEvent.Open())
	{
		return false;
	}
	if (!ShutDownEvent.Set())
	{
		return false;
	}
	return true;
}

static bool ShutdownZenServerProcess(int Pid, double MaximumWaitDurationSeconds = 25.0)
{
	const ZenServerState ServerState(/* ReadOnly */true);
	const ZenServerState::ZenServerEntry* Entry = ServerState.LookupByPid(Pid);
	uint64 ZenShutdownWaitStartTime = FPlatformTime::Cycles64();
	if (Entry)
	{
		uint16 EffectivePort = Entry->EffectiveListenPort.load(std::memory_order_relaxed);
		UE_LOGF(LogZenServiceInstance, Display, "Requesting shut down of zenserver process %d runnning on effective port %u", Pid, EffectivePort);
		if (RequestZenShutdownOnEffectivePort(EffectivePort))
		{
			while (ZenServerState::IsProcessRunning(Pid))
			{
				double ZenShutdownWaitDuration = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - ZenShutdownWaitStartTime);
				if (ZenShutdownWaitDuration < MaximumWaitDurationSeconds)
				{
					FPlatformProcess::Sleep(0.01f);
				}
				else
				{
					UE_LOGF(LogZenServiceInstance, Warning, "Timed out waiting for shut down of running service with pid %d", Pid);
					break;
				}
			}
		}
		if (!ZenServerState::IsProcessRunning(Pid))
		{
			UE_LOGF(LogZenServiceInstance, Display, "Successfully shut down zenserver process with pid %d", Pid);
			return true;
		}
	}
	else if (!ZenServerState::IsProcessRunning(Pid))
	{
		UE_LOGF(LogZenServiceInstance, Display, "Found no active process with pid %d", Pid);
		return true;
	}

	// There may be no entry left in the state table, but the process might be wrapping up its shutdown sequence.
	// Allow use of the remaining wait duration for the process to finish exiting gracefully.
	while (ZenServerState::IsProcessRunning(Pid))
	{
		double ZenShutdownWaitDuration = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - ZenShutdownWaitStartTime);
		if (ZenShutdownWaitDuration < MaximumWaitDurationSeconds)
		{
			FPlatformProcess::Sleep(0.01f);
		}
		else
		{
			UE_LOGF(LogZenServiceInstance, Warning, "Timed out waiting for shut down of running service with pid %d", Pid);
			break;
		}
	}

	if (ZenServerState::IsProcessRunning(Pid))
	{
		UE_LOGF(LogZenServiceInstance, Display, "After failed shut down request, attempting forced termination of process with pid %d", Pid);
		if (ZenServerState::Terminate(Pid))
		{
			UE_LOGF(LogZenServiceInstance, Warning, "After failed shut down request, forced to terminate process with pid %d", Pid);
			return true;
		}
	}

	if (ZenServerState::IsProcessRunning(Pid))
	{
		UE_LOGF(LogZenServiceInstance, Warning, "Failed to forcefully terminate process with pid %d", Pid);
		return false;
	}
	UE_LOGF(LogZenServiceInstance, Display, "No longer any active process with pid %d", Pid);
	return true;
}

static bool ShutDownZenServerProcessExecutable(const FString& ExecutablePath, double MaximumWaitDurationSeconds = 25.0)
{
	uint64 ZenShutdownWaitStartTime = FPlatformTime::Cycles64();
	uint32_t Pid = 0;
	while (ZenServerState::FindRunningProcessId(*ExecutablePath, &Pid))
	{
		UE_LOGF(LogZenServiceInstance, Display, "Attempting to shut down zenserver executable '%ls' process with pid %d", *ExecutablePath, Pid);
		double ZenShutdownWaitDuration = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - ZenShutdownWaitStartTime);
		if (ShutdownZenServerProcess(Pid, MaximumWaitDurationSeconds - ZenShutdownWaitDuration))
		{
			return true;
		}
		else
		{
			UE_LOGF(LogZenServiceInstance, Warning, "Failed to shut down zenserver executable '%ls' process with pid %d", *ExecutablePath, Pid);
			return false;
		}
	}
	return true;
}

static bool ShutDownZenServerProcessLockingDataDir(const FString& DataPath, double MaximumWaitDurationSeconds = 25.0)
{
	const FString LockFilePath = FPaths::Combine(DataPath, TEXT(".lock"));

	uint64 ZenShutdownWaitStartTime = FPlatformTime::Cycles64();
	if (!ZenLockFileData::IsLockFileLocked(*LockFilePath, true))
	{
		UE_LOGF(LogZenServiceInstance, Display, "Lock file '%ls' is not active, nothing to do", *LockFilePath);
		return true;
	}
	ZenLockFileData LockFileState = ZenLockFileData::ReadCbLockFile(*LockFilePath);
	if (!LockFileState.IsValid)
	{
		while (true)
		{
			if (!ZenLockFileData::IsLockFileLocked(*LockFilePath, true))
			{
				return true;
			}
			uint32_t Pid = 0;
			if (!ZenServerState::FindRunningProcessId(*GetServiceExecutableName(), &Pid))
			{
				if (!ZenLockFileData::IsLockFileLocked(*LockFilePath, true))
				{
					UE_LOGF(LogZenServiceInstance, Display, "Lock file '%ls' is no longer active, nothing to do", *LockFilePath);
					return true;
				}
				UE_LOGF(LogZenServiceInstance, Warning, "Failed to find zenserver process locking file '%ls'", *LockFilePath);
				return false;
			}
			UE_LOGF(LogZenServiceInstance, Warning, "Found locked but invalid lock file at '%ls', attempting shut down of zenserver process with pid %d", *LockFilePath, Pid);
			double ZenShutdownWaitDuration = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - ZenShutdownWaitStartTime);
			if (!ShutdownZenServerProcess(Pid, MaximumWaitDurationSeconds - ZenShutdownWaitDuration))
			{
				break;
			}
		}
		if (!ZenLockFileData::IsLockFileLocked(*LockFilePath))
		{
			UE_LOGF(LogZenServiceInstance, Display, "Successfully shut down zenserver using lock file '%ls'", *LockFilePath);
			return true;
		}
		UE_LOGF(LogZenServiceInstance, Warning, "Failed to shut down zenserver process locking file '%ls'", *LockFilePath);
		return false;
	}

	uint16 EffectivePort = LockFileState.EffectivePort;

	const ZenServerState ServerState(/* ReadOnly */true);
	const ZenServerState::ZenServerEntry* Entry = ServerState.LookupByEffectiveListenPort(EffectivePort);
	if (Entry)
	{
		UE_LOGF(LogZenServiceInstance, Display, "Requesting shut down of zenserver process using lock file '%ls' with effective port %d", *LockFilePath, EffectivePort);
		if (RequestZenShutdownOnEffectivePort(EffectivePort))
		{
			while (ZenLockFileData::IsLockFileLocked(*LockFilePath, true))
			{
				double ZenShutdownWaitDuration = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - ZenShutdownWaitStartTime);
				if (ZenShutdownWaitDuration < MaximumWaitDurationSeconds)
				{
					FPlatformProcess::Sleep(0.01f);
				}
				else
				{
					UE_LOGF(LogZenServiceInstance, Warning, "Timed out waiting for shut down of zenserver process using lock file '%ls' with effective port %u", *LockFilePath, EffectivePort);
					break;
				}
			}
			if (!ZenLockFileData::IsLockFileLocked(*LockFilePath, true))
			{
				UE_LOGF(LogZenServiceInstance, Display, "Successfully shut down zenserver process using lock file '%ls' with effective port %u", *LockFilePath, EffectivePort);
				return true;
			}
		}
	}

	while (true)
	{
		if (!ZenLockFileData::IsLockFileLocked(*LockFilePath, true))
		{
			return true;
		}
		uint32_t Pid = 0;
		if (!ZenServerState::FindRunningProcessId(LockFileState.ExecutablePath.IsEmpty() ? *GetServiceExecutableName() : *LockFileState.ExecutablePath, &Pid))
		{
			if (!ZenLockFileData::IsLockFileLocked(*LockFilePath, true))
			{
				UE_LOGF(LogZenServiceInstance, Display, "Lock file '%ls' is no longer active, nothing to do", *LockFilePath);
				return true;
			}
			UE_LOGF(LogZenServiceInstance, Warning, "Failed to find zenserver process locking file '%ls'", *LockFilePath);
			return false;
		}
		UE_LOGF(LogZenServiceInstance, Warning, "Found locked but invalid lock file at '%ls', attempting shut down of zenserver process with pid %d", *LockFilePath, Pid);
		double ZenShutdownWaitDuration = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - ZenShutdownWaitStartTime);
		if (!ShutdownZenServerProcess(Pid, MaximumWaitDurationSeconds - ZenShutdownWaitDuration))
		{
			break;
		}
	}

	if (!ZenLockFileData::IsLockFileLocked(*LockFilePath))
	{
		UE_LOGF(LogZenServiceInstance, Display, "Successfully shut down zenserver using lock file '%ls'", *LockFilePath);
		return true;
	}
	UE_LOGF(LogZenServiceInstance, Warning, "Failed to shut down zenserver process locking file '%ls'", *LockFilePath);
	return false;
}

static bool
IsZenProcessUsingDataDir(const TCHAR* LockFilePath, ZenLockFileData* OutLockFileData)
{
	if (ZenLockFileData::IsLockFileLocked(LockFilePath, true))
	{
		if (OutLockFileData)
		{
			// If an instance is running with this data path, check if we can use it and what port it is on
			*OutLockFileData = ZenLockFileData::ReadCbLockFile(LockFilePath);
		}
		return true;
	}
	return false;
}

static FString
DetermineCmdLineWithoutTransientComponents(const FString& DataPath, const FString& ExtraArgs, const TArray<FServicePluginSettings>& Plugins, bool bSendUnattendedBugReports, ERemoteNetworkService RemoteNetworkService, int16 OverrideDesiredPort, const FString& SecurityConfigPath)
{
	FString PlatformDataPath(DataPath);
	FPaths::MakePlatformFilename(PlatformDataPath);

	int32 SpaceIndex = INDEX_NONE;
	if (PlatformDataPath.FindChar(' ', SpaceIndex))
	{
		PlatformDataPath = TEXT("\"") + PlatformDataPath + TEXT("\"");
	}

	FString Parms;
	Parms.Appendf(TEXT("--port %d --data-dir %s"),
		OverrideDesiredPort,
		*PlatformDataPath);

	if (!ExtraArgs.IsEmpty())
	{
		Parms.AppendChar(TEXT(' '));
		Parms.Append(ExtraArgs);
	}

	if (!Plugins.IsEmpty())
	{
		Parms.AppendChar(TEXT(' '));
		Parms.Appendf(TEXT("--plugins-config \"%s\""),
			*GetServicePluginsConfigPath());
	}

	FString LogCommandLineOverrideValue;
	if (FParse::Value(FCommandLine::Get(), TEXT("ZenLogPath="), LogCommandLineOverrideValue))
	{
		if (!LogCommandLineOverrideValue.IsEmpty())
		{
			Parms.Appendf(TEXT(" --abslog \"%s\""),
				*FPaths::ConvertRelativePathToFull(LogCommandLineOverrideValue));
		}
	}

	FString CfgCommandLineOverrideValue;
	if (FParse::Value(FCommandLine::Get(), TEXT("ZenCfgPath="), CfgCommandLineOverrideValue))
	{
		if (!CfgCommandLineOverrideValue.IsEmpty())
		{
			Parms.Appendf(TEXT(" --config \"%s\""),
				*FPaths::ConvertRelativePathToFull(CfgCommandLineOverrideValue));
		}
	}

	if (!bSendUnattendedBugReports)
	{
		Parms.Append(TEXT(" --no-sentry"));
	}

	switch (RemoteNetworkService)
	{
	case ERemoteNetworkService::None:
		Parms.Append(TEXT(" --http-forceloopback"));
		break;
	case ERemoteNetworkService::Unsecured:
		break;
	case ERemoteNetworkService::GeneratedStaticKey:
		if (!SecurityConfigPath.IsEmpty())
		{
			Parms.Appendf(TEXT(" --security-config-path \"%s\""), *SecurityConfigPath);
		}
		break;
	default:
		checkf(false, TEXT("Unrecognized remote network service type: %d"), static_cast<int>(RemoteNetworkService));
		break;
	}

	return Parms;
}

bool
Private::IsLocalAutoLaunched(FStringView InstanceURL)
{
	if (!InstanceURL.IsEmpty() && !InstanceURL.Equals(TEXT("<DefaultInstance>")))
	{
		FString TempURL(InstanceURL);
		return IsLocalHost(TempURL);
	}
	return true;
}

bool
Private::GetLocalDataCachePathOverride(FString& OutDataPath)
{
	const TCHAR* AutoLaunchConfigSection = TEXT("Zen.AutoLaunch");
	FString DataPath;
	DetermineLocalDataCachePath(AutoLaunchConfigSection, DataPath);
	if (DataPath.IsEmpty())
	{
		return false;
	}
	OutDataPath = DataPath;
	return true;
}

bool
TryGetLocalServiceRunContext(FZenLocalServiceRunContext& OutContext)
{
	return OutContext.ReadFromJsonFile(*GetServiceRunContextPath());
}

bool
FZenLocalServiceRunContext::ReadFromJson(FJsonObject& JsonObject)
{
	Executable = JsonObject.GetFieldUntyped(TEXT("Executable"))->AsString();
	CommandlineArguments = JsonObject.GetFieldUntyped(TEXT("CommandlineArguments"))->AsString();
	WorkingDirectory = JsonObject.GetFieldUntyped(TEXT("WorkingDirectory"))->AsString();
	DataPath = JsonObject.GetFieldUntyped(TEXT("DataPath"))->AsString();
	bShowConsole = JsonObject.GetFieldUntyped(TEXT("ShowConsole"))->AsBool();
	if (!JsonObject.TryGetBoolField(UE::FSharedString(TEXT("LimitProcessLifetime")), bLimitProcessLifetime))
	{
		bLimitProcessLifetime = false;
	}
	return true;
}

void
FZenLocalServiceRunContext::WriteToJson(TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>& Writer) const
{
	Writer.WriteValue(TEXT("Executable"), Executable);
	Writer.WriteValue(TEXT("CommandlineArguments"), CommandlineArguments);
	Writer.WriteValue(TEXT("WorkingDirectory"), WorkingDirectory);
	Writer.WriteValue(TEXT("DataPath"), DataPath);
	Writer.WriteValue(TEXT("ShowConsole"), bShowConsole);
	Writer.WriteValue(TEXT("LimitProcessLifetime"), bLimitProcessLifetime);
}

bool
FZenLocalServiceRunContext::ReadFromJsonFile(const TCHAR* Filename)
{
	FString JsonText;
	if (!FFileHelper::LoadFileToString(JsonText, Filename))
	{
		return false;
	}

	TSharedPtr<FJsonObject> JsonObject;
	TSharedRef<TJsonReader<TCHAR>> Reader = TJsonReaderFactory<TCHAR>::Create(JsonText);
	if (!FJsonSerializer::Deserialize(Reader, JsonObject) || !JsonObject.IsValid())
	{
		return false;
	}

	return ReadFromJson(*JsonObject);
}

bool
FZenLocalServiceRunContext::WriteToJsonFile(const TCHAR* Filename) const
{
	FString JsonTcharText;
	TSharedRef<TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>> Writer = TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>::Create(&JsonTcharText);
	Writer->WriteObjectStart();
	WriteToJson(*Writer);
	Writer->WriteObjectEnd();
	Writer->Close();

	if (!FFileHelper::SaveStringToFile(JsonTcharText, Filename))
	{
		return false;
	}

	return true;
}

bool
IsLocalServiceRunning(const TCHAR* DataPath, uint16* OutEffectivePort)
{
	const FString LockFilePath = FPaths::Combine(DataPath, TEXT(".lock"));
	ZenLockFileData LockFileState;
	if (IsZenProcessUsingDataDir(*LockFilePath, &LockFileState))
	{
		if (OutEffectivePort != nullptr && LockFileState.IsValid && LockFileState.IsReady)
		{
			*OutEffectivePort = LockFileState.EffectivePort;
		}
		return true;
	}
	return false;
}

FProcHandle
StartLocalService(const FZenLocalServiceRunContext& Context, FStringView TransientArgs)
{
	FString Parms = Context.GetCommandlineArguments();
	if (Context.GetLimitProcessLifetime())
	{
		Parms.Appendf(TEXT(" --owner-pid %d"), FPlatformProcess::GetCurrentProcessId());
	}

	if (!TransientArgs.IsEmpty())
	{
		Parms.Appendf(TEXT(" %.*s"), TransientArgs.Len(), TransientArgs.GetData());
	}

	UE_LOGF(LogZenServiceInstance, Display, "Launching executable '%ls', working dir '%ls', data dir '%ls', args '%ls'", *Context.GetExecutable(), *Context.GetWorkingDirectory(), *Context.GetDataPath(), *Parms);

	FProcHandle Proc;
#if PLATFORM_WINDOWS
	FString PlatformExecutable = Context.GetExecutable();
	FPaths::MakePlatformFilename(PlatformExecutable);
	FString PlatformWorkingDirectory = Context.GetWorkingDirectory();
	FPaths::MakePlatformFilename(PlatformWorkingDirectory);
	{
		// We could switch to FPlatformProcess::CreateProc for Windows as well if we are able to add the CREATE_BREAKAWAY_FROM_JOB flag
		// as that is needed on CI to stop Horde from terminating the zenserver process
		STARTUPINFO StartupInfo = {
			sizeof(STARTUPINFO),
			NULL, NULL, NULL,
			(::DWORD)CW_USEDEFAULT,
			(::DWORD)CW_USEDEFAULT,
			(::DWORD)CW_USEDEFAULT,
			(::DWORD)CW_USEDEFAULT,
			(::DWORD)0, (::DWORD)0, (::DWORD)0,
			(::DWORD)STARTF_USESHOWWINDOW,
			(::WORD)(Context.GetShowConsole() ? SW_SHOWMINNOACTIVE : SW_HIDE),
			0, NULL,
			HANDLE(nullptr),
			HANDLE(nullptr),
			HANDLE(nullptr)
		};

		FString CommandLine = FString::Printf(TEXT("\"%s\" %s"), *PlatformExecutable, *Parms);
		::DWORD CreationFlagsArray[] = {
			NORMAL_PRIORITY_CLASS | DETACHED_PROCESS | CREATE_BREAKAWAY_FROM_JOB, // Try with the breakaway flag first
			NORMAL_PRIORITY_CLASS | DETACHED_PROCESS // If that fails (access denied), try without the breakaway flag next
		};

		for (::DWORD CreationFlags : CreationFlagsArray)
		{
			PROCESS_INFORMATION ProcInfo;
			if (CreateProcess(NULL, CommandLine.GetCharArray().GetData(), nullptr, nullptr, false, CreationFlags, nullptr, PlatformWorkingDirectory.GetCharArray().GetData(), &StartupInfo, &ProcInfo))
			{
				::CloseHandle(ProcInfo.hThread);
				Proc = FProcHandle(ProcInfo.hProcess);
				break;
			}
		}

		if (!Proc.IsValid())
		{
			UE_LOGF(LogZenServiceInstance, Warning, "Failed launching %ls status: %d.", *CommandLine, GetLastError());
		}
	}
#else
	{
		bool bLaunchDetached = true;
		bool bLaunchHidden = true;
		bool bLaunchReallyHidden = !Context.GetShowConsole();
		uint32* OutProcessID = nullptr;
		int32 PriorityModifier = 0;
		void* PipeWriteChild = nullptr;
		void* PipeReadChild = nullptr;
#if PLATFORM_MAC
		int devnullfd = open("/dev/null", O_WRONLY);
		void* NullPipe = [[NSFileHandle alloc] initWithFileDescriptor: devnullfd];
		PipeWriteChild = NullPipe;
		ON_SCOPE_EXIT
		{
			close([(NSFileHandle*)NullPipe fileDescriptor]);
			[(NSFileHandle*)NullPipe release];
			close(devnullfd);
		};
#elif PLATFORM_UNIX
		int devnullfd = open("/dev/null", O_WRONLY);
		void* NullPipe = new FPipeHandle(devnullfd, devnullfd);
		PipeWriteChild = NullPipe;
		ON_SCOPE_EXIT
		{
			FPipeHandle * PipeHandle = reinterpret_cast< FPipeHandle* >(NullPipe);
			delete PipeHandle;
			close(devnullfd);
		};
#endif
		Proc = FPlatformProcess::CreateProc(
			*Context.GetExecutable(),
			*Parms,
			bLaunchDetached,
			bLaunchHidden,
			bLaunchReallyHidden,
			OutProcessID,
			PriorityModifier,
			*Context.GetWorkingDirectory(),
			PipeWriteChild,
			PipeReadChild);
	}
#endif
	return Proc;
}

bool
StartLocalService(const FZenLocalServiceRunContext& Context)
{
	FString TransientParms;
	FString StartupEventName = ZenSharedEvent::GetStartupEventName();
	ZenSharedEvent StartupEvent(StartupEventName);
	if (!StartupEvent.Create())
	{
		UE_LOGF(LogZenServiceInstance, Warning, "Failed to create startup event when launcing executable '%ls'", *Context.GetExecutable());
		return false;
	}
	TransientParms.Appendf(TEXT(" --child-id %s"), *StartupEventName);
#if PLATFORM_MAC || PLATFORM_UNIX
	TransientParms.Append(TEXT(" --detach"));
#endif

	FProcHandle Proc = StartLocalService(Context, TransientParms);
	if (Proc.IsValid())
	{
		bool ZenServerIsReady = false;
		FScopedSlowTask WaitForZenReadySlowTask(0, NSLOCTEXT("Zen", "Zen_WaitingForReady", "Waiting for ZenServer to be ready"));
		uint64 ZenWaitStartTime = FPlatformTime::Cycles64();

		enum class EWaitDurationPhase
		{
			Short,
			Medium,
			Long
		} DurationPhase = EWaitDurationPhase::Short;

		while (FPlatformProcess::IsProcRunning(Proc))
		{
			if (StartupEvent.Wait(5000))
			{
				ZenServerIsReady = true;
				break;
			}

			double ZenWaitDuration = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - ZenWaitStartTime);
			if (ZenWaitDuration >= 10.0)
			{
				if (DurationPhase == EWaitDurationPhase::Short)
				{
					if (!FPlatformProcess::IsProcRunning(Proc))
					{
#if !IS_PROGRAM
						if (!FApp::IsUnattended() && !IsRunningCommandlet() && !GIsRunningUnattendedScript)
						{
							FText ZenLaunchFailurePromptTitle = NSLOCTEXT("Zen", "Zen_LaunchFailurePromptTitle", "Failed to launch");

							FFormatNamedArguments FormatArguments;
							FString LogFilePath = FPaths::Combine(Context.GetDataPath(), TEXT("logs"), TEXT("zenserver.log"));
							FPaths::MakePlatformFilename(LogFilePath);
							FormatArguments.Add(TEXT("LogFilePath"), FText::FromString(LogFilePath));
							FText ZenLaunchFailurePromptText = FText::Format(NSLOCTEXT("Zen", "Zen_LaunchFailurePromptText", "Unreal Zen Storage Server failed to launch. Please check the ZenServer log file for details:\n{LogFilePath}"), FormatArguments);
							FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *ZenLaunchFailurePromptText.ToString(), *ZenLaunchFailurePromptTitle.ToString());
							break;
						}
						else
#endif
						{
							// Just log as there is no one to show a message
							UE_LOGF(LogZenServiceInstance, Warning, "Unreal Zen Storage Server did not launch in the expected duration");
							break;
						}
					}

					// Note that the dialog may not show up when zenserver is needed early in the launch cycle, but this will at least ensure
					// the splash screen is refreshed with the appropriate text status message.
					WaitForZenReadySlowTask.MakeDialog(true, false);
					UE_LOGF(LogZenServiceInstance, Display, "Waiting for ZenServer to be ready...");
					DurationPhase = EWaitDurationPhase::Medium;
				}
#if !IS_PROGRAM
				else if (!(FApp::IsUnattended() || IsRunningCommandlet() || GIsRunningUnattendedScript) && ZenWaitDuration > 20.0 && (DurationPhase == EWaitDurationPhase::Medium))
				{
					FText ZenLongWaitPromptTitle = NSLOCTEXT("Zen", "Zen_LongWaitPromptTitle", "Wait for ZenServer?");
					FText ZenLongWaitPromptText = NSLOCTEXT("Zen", "Zen_LongWaitPromptText", "Unreal Zen Storage Server is taking a long time to launch. It may be performing maintenance. Keep waiting?");
					if (FPlatformMisc::MessageBoxExt(EAppMsgType::YesNo, *ZenLongWaitPromptText.ToString(), *ZenLongWaitPromptTitle.ToString()) == EAppReturnType::No)
					{
						break;
					}
					DurationPhase = EWaitDurationPhase::Long;
				}
#endif
				if (WaitForZenReadySlowTask.ShouldCancel())
				{
					break;
				}
			}
		}

		if (!ZenServerIsReady)
		{
			if (FPlatformProcess::IsProcRunning(Proc))
			{
				UE_LOGF(LogZenServiceInstance, Warning, "Terminating unresponsive process for executable '%ls'", *Context.GetExecutable());
				FPlatformProcess::TerminateProc(Proc, true);
			}
		}
		FPlatformProcess::CloseProc(Proc);
		return ZenServerIsReady;
	}
	return false;
}
	
bool
StopLocalService(const TCHAR* DataPath, double MaximumWaitDurationSeconds)
{
	const FString LockFilePath = FPaths::Combine(DataPath, TEXT(".lock"));
	ZenLockFileData LockFileState;
	if (ZenLockFileData::IsLockFileLocked(*LockFilePath, true))
	{
		return ShutDownZenServerProcessLockingDataDir(DataPath, MaximumWaitDurationSeconds);
	}
	return true;
}

FString
GetLocalServiceInstallPath()
{
	if (FZenServiceLink Link = FZenServiceLink::Read(GetServiceLinkPath()); Link)
	{
		return Link.ServicePath;
	}
	else
	{
		return GetServiceCopyInstallPath();
	}
}

FString
GetLocalInstallUtilityPath()
{
	if (FZenServiceLink Link = FZenServiceLink::Read(GetServiceLinkPath()); Link)
	{
		return Link.UtilityPath;
	}
	else
	{
		return GetUtilityCopyInstallPath();
	}
}

FString
GetLocalServiceInstallVersion(bool bDetailed)
{
	IFileManager& FileManager = IFileManager::Get();
	if (FZenServiceLink Link = FZenServiceLink::Read(GetServiceLinkPath()); Link)
	{
		if (!FileManager.FileExists(*Link.ServicePath))
		{
			return FZenVersion().ToString(bDetailed);
		}
		return Link.Version.ToString(bDetailed);
	}
	else
	{
		const FString ServicePath = GetServiceCopyInstallPath();

		if (!FileManager.FileExists(*ServicePath))
		{
			return FZenVersion().ToString(bDetailed);
		}

		FZenVersion InstallVersion = GetZenVersion(GetUtilityCopyInstallPath(), ServicePath, GetInstallVersionCachePath());
		return InstallVersion.ToString(bDetailed);
	}
}

static bool GIsDefaultServicePresent = false;

FZenServiceInstance& GetDefaultServiceInstance()
{
	static FZenServiceInstance DefaultServiceInstance;
	GIsDefaultServicePresent = true;
	return DefaultServiceInstance;
}

bool IsDefaultServicePresent()
{
	return GIsDefaultServicePresent;
}

FScopeZenService::FScopeZenService()
	: FScopeZenService(FStringView())
{
}

FScopeZenService::FScopeZenService(FStringView InstanceURL)
{
	if (!InstanceURL.IsEmpty() && !InstanceURL.Equals(TEXT("<DefaultInstance>")))
	{
		UniqueNonDefaultInstance = MakeUnique<FZenServiceInstance>(InstanceURL);
		ServiceInstance = UniqueNonDefaultInstance.Get();
	}
	else
	{
		ServiceInstance = &GetDefaultServiceInstance();
	}
}

FScopeZenService::FScopeZenService(FServiceSettings&& InSettings)
{
	UniqueNonDefaultInstance = MakeUnique<FZenServiceInstance>(MoveTemp(InSettings));
	ServiceInstance = UniqueNonDefaultInstance.Get();
}

FScopeZenService::~FScopeZenService()
{}

FZenServiceInstance::FZenServiceInstance()
: FZenServiceInstance(FStringView())
{
}

FZenServiceInstance::FZenServiceInstance(FStringView InstanceURL)
{
	if (InstanceURL.IsEmpty())
	{
		Settings.ReadFromConfig();
		if (Settings.IsAutoLaunch())
		{
			// Ensure that the zen data path is inherited by subprocesses
			FPlatformMisc::SetEnvironmentVar(TEXT("UE-ZenSubprocessDataPath"), *Settings.SettingsVariant.Get<FServiceAutoLaunchSettings>().DataPath);
		}
	}
	else
	{
		Settings.ReadFromURL(InstanceURL);
	}

	Initialize();
}

FZenServiceInstance::FZenServiceInstance(FServiceSettings&& InSettings)
: Settings(MoveTemp(InSettings))
{
	Initialize();
}

FZenServiceInstance::~FZenServiceInstance()
{
}

const FString FZenServiceInstance::GetPath() const
{
	if (Settings.IsAutoLaunch())
	{
		return Settings.SettingsVariant.Get<FServiceAutoLaunchSettings>().DataPath;
	}
	return GetEndpoint().GetURL();
}

bool
FZenServiceInstance::IsServiceRunning()
{
	return !Settings.IsAutoLaunch() || bHasLaunchedLocal || bHasLaunchedSystemService;
}

bool
FZenServiceInstance::IsServiceReady()
{
	return PingService(2, /*Silent*/ false);
}

bool 
FZenServiceInstance::PingService(uint32_t AttemptCount, bool Silent)
{
	// 10 minute duration timeout per ping.  If the server is wiping state,
	// we may have to wait a long time for it after startup.
	const uint32 DefaultPingOperationTimeoutMS = 600000;
	uint32 Attempt = 0;
	while (IsServiceRunning())
	{
		Attempt++;

		Zen::FZenHttpRequest Request(Endpoint, false, DefaultSmallOperationConnectTimeoutMS, DefaultPingOperationTimeoutMS);
		Zen::FZenHttpRequest::Result Result = Request.PerformBlockingDownload(TEXTVIEW("health/ready"), nullptr, Zen::EContentType::Text);
		
		if (Result == Zen::FZenHttpRequest::Result::Success && Zen::IsSuccessCode(Request.GetResponseCode()))
		{
			if (!Silent)
			{
				UE_LOGFMT(LogZenServiceInstance, Display, "Unreal Zen Storage Server HTTP service at {Domain} status: {Response}.", Endpoint.GetURL(), Request.GetResponseAsString());
			}
			return true;
		}

		if (IsServiceRunningLocally())
		{
			if (Attempt == AttemptCount)
			{
				if (!Silent)
				{
					UE_LOGFMT(LogZenServiceInstance, Display, "Unable to reach Local Unreal Zen Storage Server HTTP service at {Domain}: {Result}", Endpoint.GetURL(), Request.GetDetailedResultString(Endpoint.GetURL()));
				}
				break;
			}
		}
		else
		{
			if (!Silent)
			{
				UE_LOGFMT(LogZenServiceInstance, Display, "Unable to reach Unreal Zen Storage Server HTTP service at {Domain}: {Result}", Endpoint.GetURL(), Request.GetDetailedResultString(Endpoint.GetURL()));
			}
			break;
		}
	}
	return false;
}

bool 
FZenServiceInstance::TryRecovery(ERecoveryMode RecoveryMode)
{
	if (!bHasLaunchedLocal)
	{
		return false;
	}

	if (PingService(1, /*Silent*/ true))
	{
		UE_LOGF(LogZenServiceInstance, Verbose, "Local ZenServer status: Healthy. Skipping recovery");
		return true;
	}

	const FString ExecutablePath = GetLocalServiceInstallPath();
	const FString ExecutionContextFilePath = GetServiceRunContextPath();

	static std::atomic<int64> LastRecoveryTicks;
	static bool bLastRecoveryResult = false;
	const FTimespan MaximumWaitForHealth = FTimespan::FromSeconds(30);
	const FTimespan MinimumDurationSinceLastRecovery = FTimespan::FromMinutes(2);

	FTimespan TimespanSinceLastRecovery = FDateTime::UtcNow() - FDateTime(LastRecoveryTicks.load(std::memory_order_relaxed));

	if (RecoveryMode == ERecoveryMode::Hard || (TimespanSinceLastRecovery > MinimumDurationSinceLastRecovery))
	{
		const uint32_t MaxAttempts = RecoveryMode == ERecoveryMode::Hard ? 3 : 1;
		uint32_t Attempts = 0;
		for(;;)
		{
			Attempts++;
			FSystemWideCriticalSection RecoveryCriticalSection(TEXT("ZenServerLaunch"), MaximumWaitForHealth);
			if (!RecoveryCriticalSection.IsValid())
			{
				// A recovery is already in progress but did not complete in time, we assume we failed and let recovery continue on a different thread
				if (PingService(1, /*Silent*/ true))
				{
					return true;
				}
				if (Attempts == MaxAttempts)
				{
					UE_LOGF(LogZenServiceInstance, Verbose, "Waiting for other process to complete recovery failed, abandonding recovery attempt");
					return false;
				}
				else
				{
					UE_LOGF(LogZenServiceInstance, Verbose, "Waiting for other process to complete recovery failed, retrying...");
					continue;
				}
			}

			// We test if the service is healthy as a different process might already have triggered a recovery
			bLastRecoveryResult = PingService(1, /*Silent*/ false);
			if (!bLastRecoveryResult)
			{
				const uint64_t RecoveryStartTime = FPlatformTime::Cycles64();

				// Update timespan since it may have changed since we waited to enter the crit section
				TimespanSinceLastRecovery = FDateTime::UtcNow() - FDateTime(LastRecoveryTicks.load(std::memory_order_relaxed));
				if (RecoveryMode == ERecoveryMode::Hard || (TimespanSinceLastRecovery > MinimumDurationSinceLastRecovery))
				{
					UE_LOGF(LogZenServiceInstance, Display, "Local ZenServer is unreachable, initiating recovery...");

					uint16 Port = GetEndpoint().GetPort();

					bool bShutdownExistingInstance = true;
					{
						const ZenServerState ServerState(/* ReadOnly */true);
						const ZenServerState::ZenServerEntry* Entry = ServerState.LookupByEffectiveListenPort(Port);
						if (Entry)
						{
							if (Entry->Pid.load(std::memory_order_relaxed) != AutoLaunchedPid)
							{
								// The running process pid is not the same as the one we launched.  The process was relaunched elsewhere. Avoid shutting it down again.
								bShutdownExistingInstance = false;
							}
						}
					}
					if (bShutdownExistingInstance && !ShutdownZenServerProcess((int)AutoLaunchedPid))	// !ShutdownRunningServiceUsingEffectivePort(Port))
					{
						UE_LOGF(LogZenServiceInstance, Warning, "Local ZenServer recovery failed to shut down running instance with pid %d", AutoLaunchedPid);
						return false;
					}

					AutoLaunch(Settings.SettingsVariant.Get<FServiceAutoLaunchSettings>(), ExecutablePath, ExecutionContextFilePath, Endpoint);
					Port = GetEndpoint().GetPort();

					FDateTime StartedWaitingForHealth = FDateTime::UtcNow();
					bLastRecoveryResult = PingService(1, /*Silent*/ true);
					while (!bLastRecoveryResult)
					{
						FTimespan WaitForHealth = FDateTime::UtcNow() - StartedWaitingForHealth;
						if (WaitForHealth > MaximumWaitForHealth)
						{
							UE_LOGF(LogZenServiceInstance, Warning, "Local ZenServer recovery timed out waiting for service to become healthy");
							break;
						}

						FPlatformProcess::Sleep(0.5f);
						if (!IsZenProcessUsingEffectivePort(Port))
						{
							AutoLaunch(Settings.SettingsVariant.Get<FServiceAutoLaunchSettings>(), ExecutablePath, ExecutionContextFilePath, Endpoint);
						}
						bLastRecoveryResult = PingService(1, /*Silent*/ false);
					}

					double ZenRecoveryDuration = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - RecoveryStartTime);
					if (bLastRecoveryResult)
					{
						UE_LOGF(LogZenServiceInstance, Display, "Local ZenServer at %ls post recovery finished in %.3lf seconds, status: Healthy", *Endpoint.GetURL(), ZenRecoveryDuration);
						LastRecoveryTicks.store(0, std::memory_order_relaxed);
					}
					else
					{
						UE_LOGF(LogZenServiceInstance, Warning, "Local ZenServer at %ls post recovery finished in %.3lf seconds, status: NOT healthy", *Endpoint.GetURL(), ZenRecoveryDuration);
						LastRecoveryTicks.store(FDateTime::UtcNow().GetTicks(), std::memory_order_relaxed);
					}
				}
			}
			break;
		}
	}

	return bLastRecoveryResult;
}

bool
FZenServiceInstance::AddSponsorProcessIDs(TArrayView<uint32> SponsorProcessIDs)
{
	uint16 Port = GetEndpoint().GetPort();
	ZenServerState State(/*bReadOnly*/ false);
	ZenServerState::ZenServerEntry* Entry = State.LookupByEffectiveListenPort(Port);
	if (Entry)
	{
		bool bAllAdded = true;
		for (uint32 SponsorProcessID : SponsorProcessIDs)
		{
			if (!Entry->AddSponsorProcess(SponsorProcessID))
			{
				bAllAdded = false;
			}
		}
		return bAllAdded;
	}
	return false;
}

uint16
FZenServiceInstance::GetAutoLaunchedPort()
{
	return AutoLaunchedPort;
}

static bool UninstallSystemService(FString UtilityPath);
static void PromptUserOfFailedSystemServiceUninstall();

static bool IsDefaultInstanceSystemService()
{
	check(GConfig && GConfig->IsReadyForUse());
	const TCHAR* ConfigSection = TEXT("Zen");
	bool bSystemService = false;
	GConfig->GetBool(ConfigSection, TEXT("SystemService"), bSystemService, GEngineIni);

	return bSystemService;
}

static bool IsDefaultInstanceAutoLaunch()
{
	check(GConfig && GConfig->IsReadyForUse());
	const TCHAR* ConfigSection = TEXT("Zen");
	bool bAutoLaunch = false;
	GConfig->GetBool(ConfigSection, TEXT("AutoLaunch"), bAutoLaunch, GEngineIni);

	return bAutoLaunch;
}

void
FZenServiceInstance::Initialize()
{
	if (Settings.IsSystemService())
	{
		if (ConditionalUpdateSystemServiceInstall(Settings.SystemServiceSettings))
		{
			return;
		}

		Settings.bSystemService = false;
	}

	// Creating a FSystemWideCriticalSection can sometimes fail if we haven't created
	// ApplicationSettingsDir, as it attempts to take a file lock on a file in that
	// directory. So we ensure it's created here before using FSystemWideCriticalSection.
	IFileManager& FileManager = IFileManager::Get();
	FileManager.MakeDirectory(FPlatformProcess::ApplicationSettingsDir());

	static bool bFirstTime = true;
	if (bFirstTime)
	{
		if (!IsDefaultInstanceSystemService() && IsDefaultInstanceAutoLaunch())
		{
			FSystemWideCriticalSection InstallCriticalSection(TEXT("ZenServerInstall"), FTimespan::FromSeconds(FApp::IsUnattended() ? 30.0 : 10.0));
			if (!InstallCriticalSection.IsValid())
			{
				PromptUserOfFailedSystemServiceUninstall();
			}

			if (!UninstallSystemService(GetInTreeUtilityPath()))
			{
				PromptUserOfFailedSystemServiceUninstall();
			}
		}
		bFirstTime = false;
	}

	if (Settings.IsAutoLaunch())
	{
		uint64 ZenAutoLaunchStartTime = FPlatformTime::Cycles64();
		const FServiceAutoLaunchSettings& AutoLaunchSettings = Settings.SettingsVariant.Get<FServiceAutoLaunchSettings>();
		bool ServiceIsInstalled = ConditionalUpdateLocalInstall(AutoLaunchSettings.InstallMode);
		if (ServiceIsInstalled)
		{
			const FString ExecutablePath = GetLocalServiceInstallPath();
			const FString ExecutionContextFilePath = GetServiceRunContextPath();

			int LaunchAttempts = 0;
			const FTimespan MaximumWaitForHealth = FTimespan::FromSeconds(30);

			FDateTime StartedWaitingForHealth = FDateTime::UtcNow();
			while (true)
			{
				{
					FSystemWideCriticalSection RecoveryCriticalSection(TEXT("ZenServerLaunch"), FTimespan::FromSeconds(FApp::IsUnattended() ? 15.0 : 5.0));
					if (!RecoveryCriticalSection.IsValid())
					{
						FTimespan WaitForRecovery = FDateTime::UtcNow() - StartedWaitingForHealth;
						UE_LOGF(LogZenServiceInstance, Display, "Local ZenServer AutoLaunch initialization still waiting for other process to complete startup/recovery, waited %.3lf seconds", WaitForRecovery.GetTotalSeconds());
					}
					else
					{
						bool bSecurityConfigChanged = ConditionalUpdateSecurityConfig(AutoLaunchSettings.RemoteNetworkService, GetLocalInstallPath());
						if (bSecurityConfigChanged)
						{
							ShutDownZenServerProcessLockingDataDir(AutoLaunchSettings.DataPath);
						}

						uint16 Port = GetEndpoint().GetPort();

						bHasLaunchedLocal = AutoLaunch(AutoLaunchSettings, ExecutablePath, ExecutionContextFilePath, Endpoint);
						if (bHasLaunchedLocal)
						{
							const ZenServerState State(/*ReadOnly*/true);
							const ZenServerState::ZenServerEntry* RunningEntry = State.LookupByEffectiveListenPort(Port);
							if (RunningEntry != nullptr)
							{
								AutoLaunchedPid = RunningEntry->Pid.load(std::memory_order_relaxed);
							}
							AutoLaunchedPort = GetEndpoint().GetPort();
							bIsRunningLocally = true;
							if (IsServiceReady())
							{
								break;
							}
							else
							{
								UE_LOGF(LogZenServiceInstance, Display, "Local ZenServer AutoLaunch completed but unable to reach service at %ls", *Endpoint.GetURL());
							}
						}
					}
				}

				++LaunchAttempts;
				FTimespan WaitForHealth = FDateTime::UtcNow() - StartedWaitingForHealth;
				if ((WaitForHealth > MaximumWaitForHealth) && (LaunchAttempts > 2))
				{
					bHasLaunchedLocal = false;
					bIsRunningLocally = false;
					UE_LOGF(LogZenServiceInstance, Warning, "Local ZenServer AutoLaunch initialization timed out waiting for service to become healthy, waited %.3lf seconds", WaitForHealth.GetTotalSeconds());
					break;
				}
				UE_LOGF(LogZenServiceInstance, Log, "Awaiting ZenServer readiness, waited for %.3lf seconds", WaitForHealth.GetTotalSeconds());
				FPlatformProcess::Sleep(0.5f);
			}
		}
		double ZenAutoLaunchDuration = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - ZenAutoLaunchStartTime);
		UE_LOGF(LogZenServiceInstance, Log, "Local ZenServer AutoLaunch initialization completed in %.3lf seconds", ZenAutoLaunchDuration);
	}
	else
	{
		const FServiceConnectSettings& ConnectExistingSettings = Settings.SettingsVariant.Get<FServiceConnectSettings>();
		bIsRunningLocally = IsLocalHost(ConnectExistingSettings.HostName);
		Endpoint = FZenServiceEndpoint(ConnectExistingSettings.HostName, ConnectExistingSettings.Port);
	}
}

static void
PromptUserNeedsElevatedPermissions()
{
#if !IS_PROGRAM
	if (!FApp::IsUnattended() && !IsRunningCommandlet() && !GIsRunningUnattendedScript)
	{
		FText ZenPermissionsPromptTitle = NSLOCTEXT("Zen", "Zen_PermissionsPromptTitle", "Administrator permissions required");
		FText ZenPermissionsPromptText = NSLOCTEXT("Zen", "Zen_PermissionsPromptText", "Unable to update Unreal Zen Storage Server. Updating the installed system service requires administrator privileges.");
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *ZenPermissionsPromptText.ToString(), *ZenPermissionsPromptTitle.ToString());
	}
	else
#endif
	{
		// Just log as there is no one to show a message
		UE_LOGF(LogZenServiceInstance, Warning, "Unable to update Unreal Zen Storage Server. Updating the installed system service requires administrator privileges.");
	}
}

static void
PromptUserToStopRunningServerInstanceForUpdate(const FString& ServerFilePath)
{
#if !IS_PROGRAM
	if (!FApp::IsUnattended() && !IsRunningCommandlet() && !GIsRunningUnattendedScript)
	{
		FText ZenUpdatePromptTitle = NSLOCTEXT("Zen", "Zen_UpdatePromptTitle", "Update required");
		FText ZenUpdatePromptText = FText::Format(NSLOCTEXT("Zen", "Zen_UpdatePromptText", "Unreal Zen Storage Server needs to be updated to a new version. Please shut down Unreal Editor and any tools that are using the ZenServer at '{0}'"), FText::FromString(ServerFilePath));
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *ZenUpdatePromptText.ToString(), *ZenUpdatePromptTitle.ToString());
	}
	else
#endif
	{
		// Just log as there is no one to show a message
		UE_LOGF(LogZenServiceInstance, Display, "Unreal Zen Storage Server needs to be updated to a new version. Please shut down any tools that are using the ZenServer at '%ls'", *ServerFilePath);
	}
}

static void
PromptUserOfLockedDataFolder(const FString& DataPath)
{
#if !IS_PROGRAM
	if (!FApp::IsUnattended() && !IsRunningCommandlet() && !GIsRunningUnattendedScript)
	{
		FText ZenLaunchFailurePromptTitle = NSLOCTEXT("Zen", "Zen_NonLocalProcessUsesDataDirPromptTitle", "Failed to launch");
		FText ZenLaunchFailurePromptText = FText::Format(NSLOCTEXT("Zen", "Zen_NonLocalProcessUsesDataDirPromptText", "Unreal Zen Storage Server Failed to auto launch, an unknown process is locking the data folder '{0}'"), FText::FromString(DataPath));
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *ZenLaunchFailurePromptText.ToString(), *ZenLaunchFailurePromptTitle.ToString());
	}
	else
#endif
	{
		// Just log as there is no one to show a message
		UE_LOGF(LogZenServiceInstance, Warning, "Unreal Zen Storage Server Failed to auto launch, an unknown process is locking the data folder '%ls'", *DataPath);
	}
}

static void
PromptUserOfFailedShutDownOfExistingProcess(uint16 Port)
{
#if !IS_PROGRAM
	if (!FApp::IsUnattended() && !IsRunningCommandlet() && !GIsRunningUnattendedScript)
	{
		FText ZenLaunchFailurePromptTitle = NSLOCTEXT("Zen", "Zen_ShutdownFailurePromptTitle", "Failed to launch");
		FText ZenLaunchFailurePromptText = FText::Format(NSLOCTEXT("Zen", "Zen_ShutdownFailurePromptText", "Unreal Zen Storage Server Failed to auto launch, failed to shut down currently running service using port '{0}'"), FText::AsNumber(Port, &FNumberFormattingOptions::DefaultNoGrouping()));
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *ZenLaunchFailurePromptText.ToString(), *ZenLaunchFailurePromptTitle.ToString());
	}
	else
#endif
	{
		// Just log as there is no one to show a message
		UE_LOGF(LogZenServiceInstance, Warning, "Unreal Zen Storage Server Failed to auto launch, failed to shut down currently running service using port %u", Port);
	}
}

static void
PromptUserOfFailedSystemServiceInstall()
{
#if !IS_PROGRAM
	if (!FApp::IsUnattended() && !IsRunningCommandlet() && !GIsRunningUnattendedScript)
	{
		FText ZenInstallSystemServiceFailurePromptTitle = NSLOCTEXT("Zen", "Zen_SystemServiceInstallFailurePromptTitle", "Failed to install system service");
		FText ZenInstallSystemServiceFailurePromptText = NSLOCTEXT("Zen", "Zen_SystemServiceInstallFailurePromptText", "Unreal Zen Storage Server failed to install as a system service. Zen will run using AutoLaunch or ConnectExisting settings.");
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *ZenInstallSystemServiceFailurePromptText.ToString(), *ZenInstallSystemServiceFailurePromptTitle.ToString());
	}
	else
#endif
	{
		// Just log as there is no one to show a message
		UE_LOGF(LogZenServiceInstance, Warning, "Unreal Zen Storage Server failed to install as a system service. Zen will run using AutoLaunch or ConnectExisting settings.");
	}
}

static void
PromptUserFailedToSetDataPathPermissions()
{
#if !IS_PROGRAM
	if (!FApp::IsUnattended() && !IsRunningCommandlet() && !GIsRunningUnattendedScript)
	{
		FText ZenDataPathPermissionsFailurePromptTitle = NSLOCTEXT("Zen", "Zen_DataPathPermissionsFailurePromptTitle", "Failed to install system service");
		FText ZenDataPathPermissionsFailurePromptText = NSLOCTEXT("Zen", "Zen_DataPathPermissionsFailurePromptText", "Failed to update permissions on existing data path for service user. Zen will run using AutoLaunch or ConnectExisting settings.");
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *ZenDataPathPermissionsFailurePromptTitle.ToString(), *ZenDataPathPermissionsFailurePromptText.ToString());
	}
	else
#endif
	{
		// Just log as there is no one to show a message
		UE_LOGF(LogZenServiceInstance, Warning, "Failed to update permissions on existing data path for service user. Zen will run using AutoLaunch or ConnectExisting settings.");
	}
}

static void
PromptUserOfFailedSystemServiceUninstall()
{
#if !IS_PROGRAM
	if (!FApp::IsUnattended() && !IsRunningCommandlet() && !GIsRunningUnattendedScript)
	{
		FText ZenUninstallSystemServiceFailurePromptTitle = NSLOCTEXT("Zen", "Zen_SystemServiceUninstallFailurePromptTitle", "Failed to uninstall system service");
		FText ZenUninstallSystemServiceFailurePromptText = NSLOCTEXT("Zen", "Zen_SystemServiceUninstallFailurePromptText", "Unreal Zen Storage Server was unable to uninstall a running system service instance. Zen AutoLaunch or ConnectExisting settings may fail to function correctly.");
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *ZenUninstallSystemServiceFailurePromptText.ToString(), *ZenUninstallSystemServiceFailurePromptTitle.ToString());
	}
	else
#endif
	{
		// Just log as there is no one to show a message
		UE_LOGF(LogZenServiceInstance, Warning, "Unreal Zen Storage Server was unable to uninstall a running system service instance. Zen AutoLaunch settings may fail to function correctly.");
	}
}

static void
PromptUserOfMismatchedSystemServiceConfiguration(const FString& DesiredCommandLine, const FString& ActualCommandLine)
{
#if !IS_PROGRAM
	if (!FApp::IsUnattended() && !IsRunningCommandlet() && !GIsRunningUnattendedScript)
	{
		FText ZenMismatchedSystemServiceConfigurationPromptTitle = NSLOCTEXT("Zen", "Zen_SystemServiceMismatchedConfigurationPromptTitle", "Mismatched configuration for Unreal Zen Storage Server");
		FText ZenMismatchedSystemServiceConfigurationPromptText = FText::Format(NSLOCTEXT("Zen", "Zen_SystemServiceMismatchedConfigurationPromptText", "Unreal Zen Storage Server detected mismatched configuration for system service installation.\n  Your configuration requested the following command line: '{0}'.\n  The installed system service is running with the command line: '{1}'\n. To force an update to the system service configuration, use -ForceZenInstall on the command line."), FText::FromString(DesiredCommandLine), FText::FromString(ActualCommandLine));
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *ZenMismatchedSystemServiceConfigurationPromptText.ToString(), *ZenMismatchedSystemServiceConfigurationPromptTitle.ToString());
	}
	else
#endif
	{
		// Just log as there is no one to show a message
		UE_LOGF(LogZenServiceInstance, Warning, "Unreal Zen Storage Server detected mismatched configuration for system service installation. Zen will run using the existing system service configuration. To force an update to the system service configuration, use -ForceZenInstall on the command line.");
	}
}

static void
PromptUserUnableToUpdateSystemServiceBinaries(const FString& InstallPath)
{
#if !IS_PROGRAM
	if (!FApp::IsUnattended() && !IsRunningCommandlet() && !GIsRunningUnattendedScript)
	{
		FText ZenUnableToUpdateBinariesPromptTitle = NSLOCTEXT("Zen", "Zen_SystemServiceUnableToUpdateBinariesPromptTitle", "Failed to update Unreal Zen Storage Server binaries");
		FText ZenUnableToUpdateBinariesPromptText = FText::Format(NSLOCTEXT("Zen", "Zen_SystemServiceUnableToUpdateBinariesPromptText", "Unreal Zen Storage Server was unable to update the binaries at '%s'. Zen will run using AutoLaunch or ConnectExisting settings."), FText::FromString(InstallPath));
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *ZenUnableToUpdateBinariesPromptTitle.ToString(), *ZenUnableToUpdateBinariesPromptText.ToString());
	}
	else
#endif
	{
		// Just log as there is no one to show a message
		UE_LOGF(LogZenServiceInstance, Warning, "Unreal Zen Storage Server was unable to update the binaries at '%ls'.Zen will run using AutoLaunch or ConnectExisting settings.", *InstallPath);
	}
}
struct FInstalledPluginInfo
{
	FString Name;
	uint32_t Version;
	TArray<FString> References;
	TMap<FName, FString> Options;
};

static TArray<FInstalledPluginInfo>
ReadInstalledPluginInfo()
{
	FString PluginInfoContents;
	TArray<TSharedPtr<FJsonValue>> PluginInfoJson;
	if (FFileHelper::LoadFileToString(PluginInfoContents, *GetServicePluginsVersionInfoPath()))
	{
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(PluginInfoContents);
		if (!FJsonSerializer::Deserialize(Reader, PluginInfoJson))
		{
			UE_LOGF(LogZenServiceInstance, Warning, "Found malformed plugin info file at '%ls'. Recreating plugin info file.", *GetServicePluginsVersionInfoPath());
			return {};
		}

		TArray<FInstalledPluginInfo> PluginInfo;
		for (const TSharedPtr<FJsonValue>& Value : PluginInfoJson)
		{
			int32 Index = PluginInfo.Emplace();

			const TSharedPtr<FJsonObject>& Object = Value->AsObject();
			PluginInfo[Index].Name = Object->GetStringField(TEXT("name"));
			PluginInfo[Index].Version = static_cast<uint32_t>(Object->GetNumberField(TEXT("version")));

			for (const TSharedPtr<FJsonValue>& Reference : Object->GetArrayField(TEXT("references")))
			{
				PluginInfo[Index].References.Add(Reference->AsString());
			}

			for (const TSharedPtr<FJsonValue>& Option : Object->GetArrayField(TEXT("options")))
			{
				for (const TTuple<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& NameValuePair : Option->AsObject()->Values)
				{
					PluginInfo[Index].Options.Add(FName(NameValuePair.Get<0>()), NameValuePair.Get<1>()->AsString());
				}
			}
		}

		return PluginInfo;
	}
	else
	{
		UE_LOGF(LogZenServiceInstance, Display, "Unable to find existing plugin information at '%ls'. Creating a new plugin info file.", *GetServicePluginsVersionInfoPath());
		return {};
	}
}

static bool
WriteInstalledPluginInfo(const TArray<FInstalledPluginInfo>& Info)
{
	TArray<TSharedPtr<FJsonValue>> Json;

	for (const FInstalledPluginInfo& PluginInfo : Info)
	{
		TSharedPtr<FJsonObject> Object = MakeShareable(new FJsonObject);
		Object->SetStringField(TEXT("name"), PluginInfo.Name);
		Object->SetNumberField(TEXT("version"), PluginInfo.Version);

		TArray<TSharedPtr<FJsonValue>> OptionsJson;
		for (const TTuple<FName, FString>& Option : PluginInfo.Options)
		{
			TSharedPtr<FJsonObject> OptionJson = MakeShareable(new FJsonObject);
			OptionJson->SetStringField(Option.Get<0>().ToString(), Option.Get<1>());
			OptionsJson.Add(MakeShareable(new FJsonValueObject(OptionJson)));
		}

		Object->SetArrayField(TEXT("options"), OptionsJson);

		TArray<TSharedPtr<FJsonValue>> ReferencesJson;
		for (const FString& Reference : PluginInfo.References)
		{
			ReferencesJson.Add(MakeShareable(new FJsonValueString(Reference)));
		}

		Object->SetArrayField(TEXT("references"), ReferencesJson);

		Json.Add(MakeShareable(new FJsonValueObject(Object)));
	}

	FString OutputString;
	TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutputString);
	if (!FJsonSerializer::Serialize(Json, Writer))
	{
		UE_LOGF(LogZenServiceInstance, Warning, "Failed to serialize plugin version info");
		return false;
	}

	const FString OutputPath = GetServicePluginsVersionInfoPath();
	UE_LOGF(LogZenServiceInstance, Display, "Writing plugin version info to '%ls'", *OutputPath);
	return FFileHelper::SaveStringToFile(OutputString, *OutputPath, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM);
}

static uint32_t
GetPluginVersion(const FString& Path)
{
#if PLATFORM_WINDOWS
	HMODULE DllHandle = LoadLibrary(*Path);
	if (!DllHandle)
	{
		UE_LOGF(LogZenServiceInstance, Warning, "Failed to retrieve plugin info from '%ls' - unable to load library. This plugin will be skipped.", *Path);
		return static_cast<uint32_t>(-1);
	}

	ON_SCOPE_EXIT
	{
		FreeLibrary(DllHandle);
	};

	typedef void (*PfnGetTransportPluginVersion)(uint32_t* OutApiVersion, uint32_t* OutPluginVersion);
	// Intermediate cast to void to silence compiler warnings about casting a FARPROC to a different
	// function type - we know the function type is correct here
	PfnGetTransportPluginVersion GetVersionFn = (PfnGetTransportPluginVersion)(void*)GetProcAddress(DllHandle, "GetTransportPluginVersion");

	if (!GetVersionFn)
	{
		UE_LOGF(LogZenServiceInstance, Warning, "Failed to retrieve plugin version info from '%ls' - library does not implement GetTransportPluginVersion()", *Path);
		return static_cast<uint32_t>(-1);
	}

	uint32_t ApiVersion;
	uint32_t PluginVersion;

	GetVersionFn(&ApiVersion, &PluginVersion);

	return PluginVersion;
#else
	return static_cast<uint32_t>(-1);
#endif
}

// Merges desired plugin info from the set of currently installed plugins and the plugins desired by
// this workspace's config. InstalledPluginInfo will be updated with any plugins from PluginSettings
// which are not present or are present at a lower version than the one we require, and PluginSettings
// will be updated with settings for any plugins present in InstalledPluginInfo. Any plugins that
// are denylisted with Zen.Plugin.DenyList will be removed from both sets and thus the final
// installation.
static bool
UpdateInstalledPluginInfo(TArray<FInstalledPluginInfo>& InstalledPluginInfo, TArray<FServicePluginSettings>& PluginSettings)
{
	bool bNeedsUpdate = false;
	FString ReferenceDir = FPaths::ConvertRelativePathToFull(FPaths::EngineDir());

	for (FServicePluginSettings& Plugin : PluginSettings)
	{
		if (FInstalledPluginInfo* InstalledPlugin = InstalledPluginInfo.FindByPredicate([&Plugin](const FInstalledPluginInfo& Info) { return Info.Name == Plugin.Name; }))
		{
			uint32_t Version = GetPluginVersion(Plugin.AbsPath);
			if (Version == static_cast<uint32_t>(-1))
			{
				continue; // skip plugin - we've already logged a warning in GetPluginVersion()
			}

			if (Version > InstalledPlugin->Version)
			{
				UE_LOGF(LogZenServiceInstance, Display, "Updating plugin '%ls' from version '%d' to version '%d'", *Plugin.Name, InstalledPlugin->Version, Version);

				InstalledPlugin->Version = Version;
				InstalledPlugin->Options = Plugin.Options;

				Plugin.bNeedsInstall = true;
				bNeedsUpdate = true;
			}

			for (const TTuple<FName, FString>& Option : Plugin.Options)
			{
				const FString* ExistingOption = InstalledPlugin->Options.Find(Option.Get<0>());
				if (!ExistingOption || *ExistingOption != Option.Get<1>())
				{
					UE_LOGF(LogZenServiceInstance, Display, "Updating plugin installation due to new or changed value for option '%ls' : '%ls'", *(Option.Get<0>().ToString()), *Option.Get<1>());

					InstalledPlugin->Options.Add(Option.Get<0>(), Option.Get<1>());

					Plugin.bNeedsInstall = true;
					bNeedsUpdate = true;
				}
			}

			if (!InstalledPlugin->References.Contains(ReferenceDir))
			{
				InstalledPlugin->References.Add(ReferenceDir);
			}
		}
		else
		{
			FInstalledPluginInfo NewPlugin;
			uint32_t Version = GetPluginVersion(Plugin.AbsPath);
			if (Version == static_cast<uint32_t>(-1))
			{
				continue; // skip plugin - we've already logged a warning in GetPluginVersion()
			}

			NewPlugin.Name = Plugin.Name;
			NewPlugin.Version = Version;
			NewPlugin.Options = Plugin.Options;
			NewPlugin.References.Add(ReferenceDir);

			InstalledPluginInfo.Add(NewPlugin);

			Plugin.bNeedsInstall = true;
			bNeedsUpdate = true;
		}
	}

	TArray<FString> PluginsToRemove;

	// Check plugins that are installed, but not by this workspace. If we find one,
	// we remove this workspace from the workspaces referencing it. Then, if there
	// are no remaining references, we remove it - otherwise, we add it to the desired
	// plugin settings for this installation, so when we later write the plugin config
	// it's included.
	for (FInstalledPluginInfo& Info : InstalledPluginInfo)
	{
		const FServicePluginSettings* Plugin = PluginSettings.FindByPredicate([&Info](const FServicePluginSettings& P) { return P.Name == Info.Name; });
		if (!Plugin)
		{
			Info.References.Remove(ReferenceDir);
			if (Info.References.IsEmpty())
			{
				UE_LOGF(LogZenServiceInstance, Display, "Removing installed plugin '%ls' as it is no longer referenced by any workspace", *Info.Name);

				PluginsToRemove.Add(Info.Name);

				bNeedsUpdate = true;
			}
			else
			{
				FServicePluginSettings NewPlugin;
				NewPlugin.Name = Info.Name;
				NewPlugin.Options = Info.Options;

				PluginSettings.Add(NewPlugin);
			}
		}
	}

	// Check known bad versions of plugins and remove them forcibly if found.
	// Plugins can be denied using [Zen.Plugin.DenyList] in config, where values
	// in the section are an array of versions, either single versions specified
	// by +PluginName=X, or potentially open ranges of versions specified by
	// +PluginName=X-Y (for bounded ranges) or +PluginName=X- (for all versions
	// above a given version).
	const FConfigSection* DenyListSection = GConfig->GetSection(TEXT("Zen.Plugin.DenyList"), false, GEngineIni);
	if (DenyListSection)
	{
		for (const FInstalledPluginInfo& Info : InstalledPluginInfo)
		{
			TArray<FString> DenyVersions;
			if (DenyListSection->GetArray(*Info.Name, DenyVersions))
			{
				for (const FString& Version : DenyVersions)
				{
					uint32 Begin, End;
					if (!LexTryParseString(Begin, *Version))
					{
						UE_LOGF(LogZenServiceInstance, Warning, "Unable to parse plugin deny list version '%ls' for plugin '%ls'", *Version, *Info.Name);
						continue;
					}

					if (Version.Contains(TEXT("-")))
					{
						if (Version.EndsWith(TEXT("-")))
						{
							End = INT_MAX;
						}
						else
						{
							int32 DashIndex = Version.Find(TEXT("-"));
							if (!LexTryParseString(End, *Version + DashIndex + 1))
							{
								UE_LOGF(LogZenServiceInstance, Warning, "Unable to parse plugin deny list version '%ls' for plugin '%ls'", *Version, *Info.Name);
								continue;
							}
						}
					}
					else
					{
						End = Begin;
					}

					if (Info.Version >= Begin && Info.Version <= End)
					{
						UE_LOGF(LogZenServiceInstance, Display, "Removing denylisted plugin version '%d'", Info.Version);
						PluginsToRemove.Add(Info.Name);

						bNeedsUpdate = true;
					}
				}
			}
		}
	}

	InstalledPluginInfo.RemoveAll([&](const FInstalledPluginInfo& I) { return PluginsToRemove.Contains(I.Name); });
	PluginSettings.RemoveAll([&](const FServicePluginSettings& S) { return PluginsToRemove.Contains(S.Name); });
	
	return bNeedsUpdate;
}

static bool IsDirectoryWriteable(const FString& Directory)
{
	IFileManager& FileManager = IFileManager::Get();
	FString NormalizedDirectory = FPaths::ConvertRelativePathToFull(Directory);
	FPaths::NormalizeDirectoryName(NormalizedDirectory);
	FFileStatData StatData = FileManager.GetStatData(*NormalizedDirectory);

	if (StatData.bIsValid && StatData.bIsDirectory)
	{
		FString TestFilePath = NormalizedDirectory / FString::Printf(TEXT(".zen-startup-test-file-%d"), FPlatformProcess::GetCurrentProcessId());
		FArchive* TestFile = FileManager.CreateFileWriter(*TestFilePath, FILEWRITE_Silent);
		if (!TestFile)
		{
			return false;
		}

		TestFile->Close();
		delete TestFile;
		FileManager.Delete(*TestFilePath);
	}
	else if (!FileManager.MakeDirectory(*NormalizedDirectory, true))
	{
		return false;
	}

	return true;
}

static bool UpdateServiceBinaries(const FString& InTreeUtilityPath, const FString& InstalledUtilityPath, const FString& InTreeServicePath, const FString& InstalledServicePath, const FString& InTreeVersionCache, const FString& InstalledVersionCache)
{
	// Even after waiting for the process to shut down we have a tolerance for failure when overwriting the target files
	if (!AttemptFileCopyWithRetries(*InstalledServicePath, *InTreeServicePath, 5.0))
	{
		return false;
	}

	if (!AttemptFileCopyWithRetries(*InstalledUtilityPath, *InTreeUtilityPath, 5.0))
	{
		return false;
	}

	AttemptFileCopyWithRetries(*InstalledVersionCache, *InTreeVersionCache, 5.0);
	return true;
}

static bool DetermineSystemServiceInfo(const FString UtilityPath, FString& OutExecutable, FString& OutCommandLine, uint16& OutPort)
{
	FString CommandLine = TEXT("service status");
	FString Output;
	int ReturnCode;

	if (!RunZenUtility(UtilityPath, CommandLine, Output, ReturnCode) || ReturnCode != 0)
	{
		return false;
	}

	// Versions of zen that don't know about the service command will still return 0 for 'service status', leading to warnings
	// later on in this code when we fail to find the info in the output.
	if (Output.StartsWith("Unknown command"))
	{
		return false;
	}

	FString CommandLineOptionsSubstring = TEXT("CommandLineOptions:");
	int32 ArgsIndex = Output.Find(CommandLineOptionsSubstring);
	if (ArgsIndex == INDEX_NONE)
	{
		UE_LOGF(LogZenServiceInstance, Warning, "Failed to determine command line options for running Unreal Zen Storage Server instance");
		return false;
	}

	ArgsIndex += CommandLineOptionsSubstring.Len();
	int32 ArgsEnd = Output.Find("\n", ESearchCase::IgnoreCase, ESearchDir::FromStart, ArgsIndex);
	if (ArgsEnd == INDEX_NONE)
	{
		UE_LOGF(LogZenServiceInstance, Warning, "Failed to determine command line options for running Unreal Zen Storage Server instance");
		return false;
	}

	OutCommandLine = Output.Mid(ArgsIndex, ArgsEnd - ArgsIndex).TrimStartAndEnd();

	FString PortSubstring = TEXT("--port ");
	int32 PortIndex = OutCommandLine.Find(PortSubstring);
	if (PortIndex == INDEX_NONE)
	{
		UE_LOGF(LogZenServiceInstance, Warning, "Failed to determine port for running Unreal Zen Storage Server instance");
		return false;
	}
	PortIndex += PortSubstring.Len();

	int32 PortEnd = OutCommandLine.Find(" ", ESearchCase::IgnoreCase, ESearchDir::FromStart, PortIndex);
	if (PortEnd == INDEX_NONE)
	{
		UE_LOGF(LogZenServiceInstance, Warning, "Failed to determine port for running Unreal Zen Storage Server instance");
		return false;
	}

	if (!LexTryParseString(OutPort, *OutCommandLine.Mid(PortIndex, PortEnd - PortIndex)))
	{
		UE_LOGF(LogZenServiceInstance, Warning, "Failed to determine port for running Unreal Zen Storage Server instance");
		return false;
	}

	FString ExecutableSubstring = TEXT("Executable:");
	int32 ExecutableIndex = Output.Find(ExecutableSubstring);
	if (ExecutableIndex == INDEX_NONE)
	{
		UE_LOGF(LogZenServiceInstance, Warning, "Failed to determine executable for running Unreal Zen Storage Server instance");
		return false;
	}
	ExecutableIndex += ExecutableSubstring.Len();

	int32 ExecutableEnd = Output.Find("\n", ESearchCase::IgnoreCase, ESearchDir::FromStart, ExecutableIndex);
	if (ExecutableEnd == INDEX_NONE)
	{
		UE_LOGF(LogZenServiceInstance, Warning, "Failed to determine executable for running Unreal Zen Storage Server instance");
		return false;
	}

	OutExecutable = Output.Mid(ExecutableIndex, ExecutableEnd - ExecutableIndex).TrimStartAndEnd();

	return true;
}

static
bool AddCurrentProcessAsSponsor(uint16 DesiredPort)
{
	ZenServerState State(/*ReadOnly*/ false);
	ZenServerState::ZenServerEntry* RunningEntry = State.LookupByDesiredListenPort(DesiredPort);
	if (RunningEntry)
	{
		RunningEntry->AddSponsorProcess(FPlatformProcess::GetCurrentProcessId());
		return true;
	}
	else
	{
		return false;
	}
}

static
bool UninstallSystemService(FString UtilityPath)
{
	{
		FString Output;
		int32 ReturnCode;
		if (RunZenUtility(UtilityPath, TEXT("service status"), Output, ReturnCode))
		{
			// 2 = bad arguments (old zen.exe version)
			if (ReturnCode == 2)
			{
				return true;
			}
			// 1 = not installed
			if (ReturnCode == 1 && Output.Contains("is not installed"))
			{
				return true;
			}
		}
	}

	FString RunningExecutable;
	FString RunningCommandLine;
	uint16 RunningPort;

	bool bIsServiceRunning = DetermineSystemServiceInfo(UtilityPath, RunningExecutable, RunningCommandLine, RunningPort);

	FString Output;
	int32 ReturnCode;

	if (bIsServiceRunning)
	{
		const ZenServerState State(/*ReadOnly*/ true);
		const ZenServerState::ZenServerEntry* Entry = State.LookupByEffectiveListenPort(RunningPort);

		if (!Entry)
		{
			UE_LOGF(LogZenServiceInstance, Warning, "Failed to look up running service info to determine active users of service, assuming it is not running");
			return false;
		}

		if (!RunZenUtilityElevated(UtilityPath, TEXT("service stop"), Output, ReturnCode))
		{
			PromptUserOfFailedSystemServiceUninstall();
			return false;
		}

		if (ReturnCode == -1)
		{
			PromptUserNeedsElevatedPermissions();
			return false;
		}
	}

	if (!RunZenUtility(UtilityPath, TEXT("service status"), Output, ReturnCode))
	{
		PromptUserOfFailedSystemServiceUninstall();
		return false;
	}

	// Versions of zen that don't know about the service command will still return 0 for 'service status', leading to warnings
	// later on in this code when we try to uninstall.
	if (Output.Contains("Unknown command"))
	{
		return true;
	}

	// zen returns 1 for status of a service that's not installed
	if (ReturnCode == 1 && Output.Contains("is not installed"))
	{
		return true;
	}

	if (!RunZenUtilityElevated(UtilityPath, TEXT("service uninstall"), Output, ReturnCode))
	{
		PromptUserOfFailedSystemServiceUninstall();
		return false;
	}

	if (ReturnCode != 0)
	{
		PromptUserNeedsElevatedPermissions();
		return false;
	}

	return true;
}

static
bool UpdateSystemServiceDataPathPermissions(const FString& DataPath)
{
#if PLATFORM_WINDOWS
	if (!FPaths::DirectoryExists(DataPath))
	{
		// We're fine here - zen will create the data path when we're already running as NT AUTHORITY\LocalService,
		// so the service account will have the appropriate permissions
		return true;
	}

	DWORD Result = 0;
	PACL OldAcl = nullptr;
	PACL NewAcl = nullptr;
	PSECURITY_DESCRIPTOR SecurityDescriptor = nullptr;
	EXPLICIT_ACCESS Access{};

	Result = GetNamedSecurityInfo(*DataPath, SE_FILE_OBJECT, DACL_SECURITY_INFORMATION, NULL, NULL, &OldAcl, NULL, &SecurityDescriptor);
	if (Result != ERROR_SUCCESS)
	{
		UE_LOGF(LogZenServiceInstance, Warning, "Failed to update data path permissions - GetNamedSecurityInfo failed with %d", Result);
		return false;
	}

	ON_SCOPE_EXIT{ LocalFree(SecurityDescriptor); };

	Access.grfAccessMode = GRANT_ACCESS;
	Access.grfAccessPermissions = STANDARD_RIGHTS_ALL | FILE_ALL_ACCESS;
	Access.grfInheritance = SUB_CONTAINERS_AND_OBJECTS_INHERIT;
	Access.Trustee.TrusteeForm = TRUSTEE_IS_NAME;
	Access.Trustee.TrusteeType = TRUSTEE_IS_USER;
	Access.Trustee.MultipleTrusteeOperation = NO_MULTIPLE_TRUSTEE;

	TCHAR AccountName[26];
	FPlatformString::Strcpy(AccountName, TEXT("NT AUTHORITY\\LocalService"));
	Access.Trustee.ptstrName = AccountName;

	Result = SetEntriesInAcl(1, &Access, OldAcl, &NewAcl);
	if (Result != ERROR_SUCCESS)
	{
		UE_LOGF(LogZenServiceInstance, Warning, "Failed to update data path permissions - SetEntriesInAcl failed with %d", Result);
		return false;
	}

	ON_SCOPE_EXIT{ LocalFree(NewAcl); };

	TArray<TCHAR> DataPathArr(*DataPath, DataPath.Len() + 1);

	Result = SetNamedSecurityInfo(DataPathArr.GetData(), SE_FILE_OBJECT, DACL_SECURITY_INFORMATION, NULL, NULL, NewAcl, NULL);
	if (Result != ERROR_SUCCESS)
	{
		UE_LOGF(LogZenServiceInstance, Warning, "Failed to update data path permissions - SetNamedSecurityInfo failed with %d", Result);
		return false;
	}

#endif
	return true;
}

bool FZenServiceInstance::ConditionalUpdateSystemServiceInstall(const FSystemServiceSettings &InSettings)
{
	FString SystemServiceInstallPath = GetSystemServiceInstallPath(InSettings);
	FString SystemUtilityInstallPath = GetSystemUtilityInstallPath(InSettings);
	FString SystemVersionCache = GetSystemServiceVersionCachePath(InSettings);

	FString InTreeUtilityPath = GetInTreeUtilityPath();
	FString InTreeServicePath = GetInTreeServicePath();
	FString InTreeVersionCache = GetInTreeVersionCache();

	FString Output;
	int ReturnCode;

	FSystemWideCriticalSection InstallCriticalSection(TEXT("ZenServerInstall"), FTimespan::FromSeconds(FApp::IsUnattended() ? 30.0 : 10.0));
	if (!InstallCriticalSection.IsValid())
	{
		PromptUserOfFailedSystemServiceInstall();
		return false;
	}

	FString SystemInstallPath = GetSystemInstallPath(InSettings);
	bool bSecurityConfigChanged = ConditionalUpdateSecurityConfig(InSettings.RemoteNetworkService, SystemInstallPath);
	FString SecurityConfigPath = (InSettings.RemoteNetworkService == ERemoteNetworkService::GeneratedStaticKey) ? GetSecurityConfigPath(SystemInstallPath) : FString();

	FString RunningExecutable;
	FString RunningCommandLine;
	uint16 RunningPort;

	TArray<FServicePluginSettings> Plugins;
	FString DesiredCommandLine = DetermineCmdLineWithoutTransientComponents(InSettings.DataPath, InSettings.ExtraArgs, Plugins, InSettings.bSendUnattendedBugReports, InSettings.RemoteNetworkService, InSettings.DesiredPort, SecurityConfigPath);

	FZenVersion Version;
	bool bIsServiceRunning = DetermineSystemServiceInfo(InTreeUtilityPath, RunningExecutable, RunningCommandLine, RunningPort);
	bool bServiceNeedsUpdate = !bIsServiceRunning
		|| RunningCommandLine != DesiredCommandLine
		|| IsInstallVersionOutOfDate(InTreeUtilityPath, SystemUtilityInstallPath, InTreeServicePath, SystemServiceInstallPath, InTreeVersionCache, SystemVersionCache, Version);

	if (bServiceNeedsUpdate)
	{
		const ZenServerState State(/*ReadOnly*/true);
		const ZenServerState::ZenServerEntry* RunningEntry = State.LookupByDesiredListenPort(InSettings.DesiredPort);
		bool bIsServiceRunningOnPort = (bIsServiceRunning && RunningEntry && RunningEntry->EffectiveListenPort == RunningPort);
		// If an existing system service install is running on this port, 'service install' will handle
		// shutdown for us. Otherwise, we need to request that the non-system-service zen shut down manually.
		if (RunningEntry != nullptr && !bIsServiceRunningOnPort)
		{
			if (!ShutdownZenServerProcess(RunningEntry->Pid))
			{
				PromptUserOfFailedShutDownOfExistingProcess(InSettings.DesiredPort);
				return false;
			}
		}

		FString ServiceInstallCommandLine;
		ServiceInstallCommandLine.Appendf(TEXT("service install --full --install-path \"%s\""), *GetSystemInstallPath(InSettings));
#if PLATFORM_LINUX
		ServiceInstallCommandLine.Appendf(TEXT(" --user %s"), FPlatformProcess::UserName(false));
#endif

		if (!UpdateSystemServiceDataPathPermissions(InSettings.DataPath))
		{
			PromptUserFailedToSetDataPathPermissions();
			return false;
		}

		if (!RunZenUtilityElevated(InTreeUtilityPath, ServiceInstallCommandLine, Output, ReturnCode, DesiredCommandLine))
		{
			PromptUserOfFailedSystemServiceInstall();
			return false;
		}

		if (ReturnCode != 0)
		{
			PromptUserNeedsElevatedPermissions();
			return false;
		}

		bIsServiceRunning = DetermineSystemServiceInfo(SystemUtilityInstallPath, RunningExecutable, RunningCommandLine, RunningPort);
	}

	if (!bIsServiceRunning)
	{
		UE_LOGF(LogZenServiceInstance, Warning, "Unknown error starting Unreal Zen Storage Server");
		return false;
	}

	AddCurrentProcessAsSponsor(RunningPort);

	Endpoint = FZenServiceEndpoint(TEXT("[::1]"), RunningPort);

	FZenLocalServiceRunContext EffectiveRunContext;
	EffectiveRunContext.Executable = RunningExecutable;
	EffectiveRunContext.CommandlineArguments = RunningCommandLine;
	EffectiveRunContext.WorkingDirectory = SystemServiceInstallPath;
	EffectiveRunContext.DataPath = InSettings.DataPath;
	EffectiveRunContext.bShowConsole = false;
	EffectiveRunContext.bLimitProcessLifetime = false;

	EffectiveRunContext.WriteToJsonFile(*GetServiceRunContextPath());

	bHasLaunchedSystemService = true;

	return true;
}

bool
FZenServiceInstance::ConditionalUpdateLocalInstall(FServiceAutoLaunchSettings::EInstallMode InstallMode)
{
	bool bNeedsPluginUpdate = FParse::Param(FCommandLine::Get(), TEXT("ForceZenPluginsInstall"));

	// Always update the installed plugin info, as we may need to add a reference from this workspace
	// even if we don't actually need to perform a full plugin update
	TArray<FInstalledPluginInfo> InstalledPluginInfo = ReadInstalledPluginInfo();
	bNeedsPluginUpdate = UpdateInstalledPluginInfo(InstalledPluginInfo, Settings.SettingsVariant.Get<FServiceAutoLaunchSettings>().Plugins) || bNeedsPluginUpdate;
	WriteInstalledPluginInfo(InstalledPluginInfo);

	if (InstallMode == FServiceAutoLaunchSettings::EInstallMode::Link)
	{
		const FString LinkPath = GetServiceLinkPath();
		IFileManager& FileManager = IFileManager::Get();

		bool LinkIsValid = false;
		const FZenServiceLink Link = FZenServiceLink::Read(LinkPath);
		if (Link)
		{
			if (!FileManager.FileExists(*Link.ServicePath))
			{
				UE_LOGF(LogZenServiceInstance, Warning, "Found service link file '%ls' pointing to unreachable service executable '%ls'", *LinkPath, *Link.ServicePath);
			}
			else if (!FileManager.FileExists(*Link.UtilityPath))
			{
				UE_LOGF(LogZenServiceInstance, Warning, "Found service link file '%ls' pointing to unreachable utility executable '%ls'", *LinkPath, *Link.UtilityPath);
			}
			else
			{
				LinkIsValid = true;
			}
		}
		else if (FileManager.FileExists(*LinkPath))
		{
			UE_LOGF(LogZenServiceInstance, Warning, "Found invalid service link file '%ls', ignoring it", *LinkPath);
		}
		
		FString InTreeServicePath = GetInTreeServicePath();
		if (LinkIsValid && InTreeServicePath == Link.ServicePath)
		{
			// If the running process already points to our executable and we have a valid link file we are good to go,
			// unless a plugin update is required - in that case we don't need to do any shutdown/install of the service
			// itself, just write the updated plugin config and restart
			uint32_t Pid = 0;
			if (ZenServerState::FindRunningProcessId(*InTreeServicePath, &Pid))
			{
				if (bNeedsPluginUpdate)
				{
					UE_LOGF(LogZenServiceInstance, Log, "Plugin update required - stopping running service at '%ls'", *Link.ServicePath);
					ShutdownZenServerProcess(Pid);
					WriteLocalPluginsConfig();
					return true;
				}
				else
				{
					UE_LOGF(LogZenServiceInstance, Log, "Service link '%ls' pointing to '%ls', version %ls is up and running", *LinkPath, *Link.ServicePath, *Link.Version.ToString(false));
					return true;
				}
			}
		}

		FString InTreeUtilityPath = GetInTreeUtilityPath();
		FZenVersion InTreeVersion;
		if (!GetZenVersion(InTreeUtilityPath, InTreeServicePath, InTreeVersion))
		{
			checkf(false, TEXT("Unable to determine version using zen utility executable path: '%s'."), *InTreeUtilityPath);
			return false;
		}
		if (LinkIsValid)
		{
			if (Link.Version < InTreeVersion)
			{
				UE_LOGF(LogZenServiceInstance, Display, "Installing service link '%ls' to '%ls', version %ls", *LinkPath, *InTreeServicePath, *InTreeVersion.ToString(false));
			}
			else
			{
				// If the instance is running, assume it is valid and up to date
				uint32_t Pid = 0;
				if (ZenServerState::FindRunningProcessId(*Link.ServicePath, &Pid))
				{
					if (bNeedsPluginUpdate)
					{
						UE_LOGF(LogZenServiceInstance, Log, "Plugin update required - stopping running service at '%ls'", *Link.ServicePath);
						ShutdownZenServerProcess(Pid);
						WriteLocalPluginsConfig();
						return true;
					}
					else
					{
						UE_LOGF(LogZenServiceInstance, Display, "Service link '%ls' pointing to '%ls', version %ls is up to date and running", *LinkPath, *Link.ServicePath, *Link.Version.ToString(false));
						return true;
					}
				}

				// Verify that the executable pointed at is runnable and is of matching version
				FZenVersion LinkedVersion;
				if (GetZenVersion(Link.UtilityPath, Link.ServicePath, LinkedVersion))
				{
					if (LinkedVersion == Link.Version)
					{
						WriteLocalPluginsConfig();
						UE_LOGF(LogZenServiceInstance, Display, "Service link '%ls' pointing to '%ls', version %ls is up to date", *LinkPath, *Link.ServicePath, *Link.Version.ToString(false));
						return true;
					}
					else
					{
						UE_LOGF(LogZenServiceInstance, Display, "Updating service link '%ls' to '%ls', version %ls (link '%ls', version %ls does not match executable version %ls)", *LinkPath, *InTreeServicePath, *InTreeVersion.ToString(false), *Link.ServicePath, *Link.Version.ToString(false), *LinkedVersion.ToString(false));
					}
				}
				else
				{
					UE_LOGF(LogZenServiceInstance, Display, "Updating service link '%ls' to '%ls', version %ls (link '%ls', version %ls pointing to invalid executable)", *LinkPath, *InTreeServicePath, *InTreeVersion.ToString(false), *Link.ServicePath, *Link.Version.ToString(false));
				}
			}
		}

		FZenServiceLink NewLink{ .ServicePath = InTreeServicePath, .UtilityPath = InTreeUtilityPath, .Version = InTreeVersion };
		if (!FZenServiceLink::Write(NewLink, LinkPath))
		{
			UE_LOGF(LogZenServiceInstance, Warning, "Failed to update service link file '%ls'", *LinkPath);
			return false;
		}

		FString ServiceCopyInstallPath = GetServiceCopyInstallPath();
		if (FileManager.FileExists(*ServiceCopyInstallPath))
		{
			ShutDownZenServerProcessExecutable(ServiceCopyInstallPath);
		}

		WriteLocalPluginsConfig();

		TArray<FString> FilesToCleanUp{ GetUtilityCopyInstallPath(), ServiceCopyInstallPath, GetInstallVersionCachePath(), GetInstallVersionCachePath() };
#if PLATFORM_WINDOWS
		FilesToCleanUp.Add(FPaths::SetExtension(FilesToCleanUp[0], TEXT("pdb")));
		FilesToCleanUp.Add(FPaths::SetExtension(FilesToCleanUp[1], TEXT("pdb")));
#endif // PLATFORM_WINDOWS
		for (const FString& FileToCleanUp : FilesToCleanUp)
		{
			// If zenserver is still running we may fail to clean up a file. Not critical, just try again next startup
			if (!AttemptFileDeleteWithRetries(*FileToCleanUp, 1.0))
			{
				UE_LOGF(LogZenServiceInstance, Log, "Failed cleaning up file {%ls} (not critical)", *FileToCleanUp);
			}
		}

		UE_LOGF(LogZenServiceInstance, Display, "Installed service link '%ls' to '%ls', version %ls", *LinkPath, *InTreeServicePath, *InTreeVersion.ToString(false));
		return true;
	}
	else if (InstallMode == FServiceAutoLaunchSettings::EInstallMode::Copy)
	{
		FString InTreeUtilityPath = GetInTreeUtilityPath();
		FString InstallUtilityPath = GetUtilityCopyInstallPath();
		FString InTreeVersionCache = GetInTreeVersionCache();

		FString InTreeServicePath = GetInTreeServicePath();
		FString InstallServicePath = GetServiceCopyInstallPath();
		FString InstallVersionCache = GetInstallVersionCachePath();

		FZenVersion VersionToInstall;

		IFileManager& FileManager = IFileManager::Get();

		bool bMainExecutablesUpdated = false;
		if (IsInstallVersionOutOfDate(InTreeUtilityPath, InstallUtilityPath, InTreeServicePath, InstallServicePath, InTreeVersionCache, InstallVersionCache, VersionToInstall))
		{
			UE_LOGF(LogZenServiceInstance, Display, "Installing service version '%ls', from '%ls' to '%ls'", *VersionToInstall.ToString(), *InTreeServicePath, *InstallServicePath);
			if (!ShutDownZenServerProcessExecutable(InstallServicePath))
			{
				PromptUserToStopRunningServerInstanceForUpdate(InstallServicePath);
				return false;
			}

			// Even after waiting for the process to shut down we have a tolerance for failure when overwriting the target files
			if (!AttemptFileCopyWithRetries(*InstallServicePath, *InTreeServicePath, 5.0))
			{
				PromptUserToStopRunningServerInstanceForUpdate(InstallServicePath);
				return false;
			}

			if (!AttemptFileCopyWithRetries(*InstallUtilityPath, *InTreeUtilityPath, 5.0))
			{
				PromptUserToStopRunningServerInstanceForUpdate(InstallServicePath);
				return false;
			}

			AttemptFileCopyWithRetries(*InstallVersionCache, *InTreeVersionCache, 1.0);

			WriteLocalPluginsConfig();

			bMainExecutablesUpdated = true;

			UE_LOGF(LogZenServiceInstance, Display, "Installed service version '%ls' from '%ls' to '%ls'", *VersionToInstall.ToString(), *InTreeServicePath, *InstallServicePath);
		}
		else
		{
			UE_LOGF(LogZenServiceInstance, Display, "Installed service at '%ls' is up to date", *InstallServicePath);
		}

		// If we need a plugin update but not a full Zen update, we just need to shut
		// down the instance and update the plugins without performing a full install
		if (bNeedsPluginUpdate && !bMainExecutablesUpdated)
		{
			if (!ShutDownZenServerProcessExecutable(InstallServicePath))
			{
				PromptUserToStopRunningServerInstanceForUpdate(InstallServicePath);
				return false;
			}

			WriteLocalPluginsConfig();
			return true;
		}

#if PLATFORM_WINDOWS
		struct FZenExecutable
		{
			FString& InTreeFilePath;
			FString& InstallFilePath;
		};
		const FZenExecutable ZenExecutables[] = {
			// Service executable (zenserver.exe)
			{InTreeServicePath, InstallServicePath},
			// Utility executable (zen.exe)
			{InTreeUtilityPath, InstallUtilityPath},
		};
		for (const FZenExecutable& Executable : ZenExecutables)
		{
			FString InTreeSymbolFilePath = FPaths::SetExtension(Executable.InTreeFilePath, TEXT("pdb"));
			FString InstallSymbolFilePath = FPaths::SetExtension(Executable.InstallFilePath, TEXT("pdb"));

			if (FileManager.FileExists(*InTreeSymbolFilePath) && (bMainExecutablesUpdated || !FileManager.FileExists(*InstallSymbolFilePath)))
			{
				AttemptFileCopyWithRetries(*InstallSymbolFilePath, *InTreeSymbolFilePath, 1.0);
			}
		}
#endif

		FString InTreeCrashpadHandlerFilePath = GetInTreeCrashpadHandlerFilePath();
		FString InstallCrashpadHandlerFilePath = GetInstallCrashpadHandlerFilePath(InTreeCrashpadHandlerFilePath);

		if (FileManager.FileExists(*InTreeCrashpadHandlerFilePath) && (bMainExecutablesUpdated || !FileManager.FileExists(*InstallCrashpadHandlerFilePath)))
		{
			AttemptFileCopyWithRetries(*InstallCrashpadHandlerFilePath, *InTreeCrashpadHandlerFilePath, 1.0);
		}

		FString InTreeOidcTokenPath = GetInTreeOidcTokenPath();
		FString InstallOidcTokenPath = GetInstallOidcTokenPath();

		if (FileManager.FileExists(*InTreeOidcTokenPath) && (bMainExecutablesUpdated || !FileManager.FileExists(*InstallOidcTokenPath)))
		{
			if (!AttemptFileCopyWithRetries(*InstallOidcTokenPath, *InTreeOidcTokenPath, 1.0))
			{
				UE_LOGF(LogZenServiceInstance, Warning, "Failed to update OidcToken executable");
			}
		}

		const FString LinkPath = GetServiceLinkPath();
		if (FZenServiceLink Link = FZenServiceLink::Read(LinkPath); Link)
		{
			ShutDownZenServerProcessExecutable(Link.ServicePath);
			if (!AttemptFileDeleteWithRetries(*LinkPath, 1.0))
			{
				UE_LOGF(LogZenServiceInstance, Log, "Failed cleaning up file {%ls} (not critical)", *LinkPath);
			}
		}

		return true;
	}
	else
	{
		return false;
	}
}

bool
FZenServiceInstance::WriteLocalPluginsConfig()
{
#if PLATFORM_WINDOWS
	TConstArrayView<FServicePluginSettings> PluginsSettings = Settings.SettingsVariant.Get<FServiceAutoLaunchSettings>().Plugins;

	TArray<TSharedPtr<FJsonValue>> JsonArray;

	// replace this with UStructToJsonObject or similar in the future
	for (const FServicePluginSettings& PluginSettings : PluginsSettings)
	{
		FString InstallPath = FPaths::Combine(GetLocalInstallPath(), FPaths::GetCleanFilename(PluginSettings.AbsPath));

		if (PluginSettings.bNeedsInstall)
		{
			if (!AttemptFileCopyWithRetries(*InstallPath, *PluginSettings.AbsPath, 5.0))
			{
				UE_LOGF(LogZenServiceInstance, Warning, "Failed to copy plugin '%ls' to '%ls' - plugin will be skipped", *PluginSettings.AbsPath, *InstallPath);
				continue;
			}
		}

		TSharedPtr<FJsonObject> JsonObject = MakeShareable(new FJsonObject);

		JsonObject->SetStringField(TEXT("name"), InstallPath); // zen server can treat name as absolute path
		for (const TPair<FName, FString>& Option: PluginSettings.Options)
		{
			JsonObject->SetStringField(Option.Key.ToString(), Option.Value);
		}

		TSharedPtr<FJsonValueObject> JsonValueObject = MakeShareable(new FJsonValueObject(MoveTemp(JsonObject)));
		JsonArray.Add(JsonValueObject);
	}

	FString OutputString;
	TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutputString);
	if (!FJsonSerializer::Serialize(JsonArray, Writer))
	{
		return false;
	}

	const FString OutputPath = GetServicePluginsConfigPath();
	UE_LOGF(LogZenServiceInstance, Display, "Writing plugin configuration to '%ls'", *OutputPath);
	return FFileHelper::SaveStringToFile(OutputString, *OutputPath, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM);
#else
	return false;
#endif
}

bool
FZenServiceInstance::AutoLaunch(const FServiceAutoLaunchSettings& InSettings, const FString& ExecutablePath, const FString& ExecutionContextFilePath, FZenServiceEndpoint& OutEndpoint)
{
	IFileManager& FileManager = IFileManager::Get();
	const FString LockFilePath = FPaths::Combine(InSettings.DataPath, TEXT(".lock"));

	FString WorkingDirectory = FPaths::GetPath(ExecutablePath);
	FString SecurityConfigPath = (InSettings.RemoteNetworkService == ERemoteNetworkService::GeneratedStaticKey) ? GetSecurityConfigPath(GetLocalInstallPath()) : FString();

	ZenLockFileData LockFileState;
	{
		const double MaxWaitDuration = 30;	// Same wait time as recovery timeout

		uint64 ZenWaitForRunningProcessReadyStartTime = FPlatformTime::Cycles64();
		while (IsZenProcessUsingDataDir(*LockFilePath, &LockFileState) && LockFileState.IsValid && !LockFileState.IsReady)
		{
			// Server is starting up, wait for it to get ready
			double ZenWaitDuration = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - ZenWaitForRunningProcessReadyStartTime);
			if (ZenWaitDuration > MaxWaitDuration)
			{
				break;
			}
			FPlatformProcess::Sleep(0.1f);
			LockFileState = ZenLockFileData();
		}
	}

	bool bShutDownExistingInstanceForDataPath = true;
	uint32 ShutdownExistingInstanceForPid = 0;
	bool bLaunchNewInstance = true;

	if (LockFileState.IsReady)
	{
		const ZenServerState State(/*ReadOnly*/true);
		if (State.LookupByPid(LockFileState.ProcessId) == nullptr && IsZenProcessUsingDataDir(*LockFilePath, nullptr))
		{
			UE_LOGF(LogZenServiceInstance, Warning, "Found locked valid lock file '%ls' but can't find registered process (Pid: %d), will attempt shut down", *LockFilePath, LockFileState.ProcessId);
			bShutDownExistingInstanceForDataPath = true;
		}
		else
		{
			if (InSettings.bIsDefaultSharedRunContext)
			{
				FZenLocalServiceRunContext DesiredRunContext;
				DesiredRunContext.Executable = ExecutablePath;
				DesiredRunContext.CommandlineArguments = DetermineCmdLineWithoutTransientComponents(InSettings.DataPath, InSettings.ExtraArgs, InSettings.Plugins, InSettings.bSendUnattendedBugReports, InSettings.RemoteNetworkService, InSettings.DesiredPort, SecurityConfigPath);
				DesiredRunContext.WorkingDirectory = WorkingDirectory;
				DesiredRunContext.DataPath = InSettings.DataPath;
				DesiredRunContext.bShowConsole = InSettings.bShowConsole;
				DesiredRunContext.bLimitProcessLifetime = InSettings.bLimitProcessLifetime;

				FZenLocalServiceRunContext CurrentRunContext;

				bool ReadCurrentContextOK = CurrentRunContext.ReadFromJsonFile(*ExecutionContextFilePath);
				if (ReadCurrentContextOK && (DesiredRunContext == CurrentRunContext))
				{
					UE_LOGF(LogZenServiceInstance, Log, "Found existing instance running on port %u matching our settings, no actions needed", InSettings.DesiredPort);
					bLaunchNewInstance = false;
					bShutDownExistingInstanceForDataPath = false;
				}
				else
				{
					FString JsonTcharText;
					{
						TSharedRef<TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>> Writer = TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>::Create(&JsonTcharText);
						Writer->WriteObjectStart();
						Writer->WriteObjectStart("Current");
						CurrentRunContext.WriteToJson(*Writer);
						Writer->WriteObjectEnd();
						Writer->WriteObjectStart("Desired");
						DesiredRunContext.WriteToJson(*Writer);
						Writer->WriteObjectEnd();
						Writer->WriteObjectEnd();
						Writer->Close();
					}
					UE_LOGF(LogZenServiceInstance, Log, "Found existing instance running on port %u with different run context, will attempt shut down\n{%ls}", InSettings.DesiredPort, *JsonTcharText);
					bShutDownExistingInstanceForDataPath = true;
					bLaunchNewInstance = true;
				}
			}
			else
			{
				UE_LOGF(LogZenServiceInstance, Log, "Found existing instance running on port %u when not using shared context, will use it", InSettings.DesiredPort);
				bShutDownExistingInstanceForDataPath = false;
				bLaunchNewInstance = false;
			}
		}
	}
	else
	{
		const ZenServerState State(/*ReadOnly*/true);
		const ZenServerState::ZenServerEntry* RunningEntry = State.LookupByDesiredListenPort(InSettings.DesiredPort);
		if (RunningEntry != nullptr)
		{
			// It is necessary to tear down an existing zenserver running on our desired port but in a different data path because:
			// 1. zenserver won't accept port collision with itself, and will instead say "Exiting since there is already a process listening to port ..."
			// 2. When UE is changing data directories (eg: DDC path config change) we don't want to leave zenservers running on the past directories for no reason
			// Unlike other shutdown scenarios, this one can't be done based on our desired data path because the zenserver we want to shut down is running in a different data path
			UE_LOGF(LogZenServiceInstance, Log, "Found existing instance running on port %u with different data directory, will attempt shutdown", InSettings.DesiredPort);
			ShutdownExistingInstanceForPid = RunningEntry->Pid;
		}
		else
		{
			UE_LOGF(LogZenServiceInstance, Log, "No current process using the data dir found, launching a new instance");
		}
		bShutDownExistingInstanceForDataPath = false;
		bLaunchNewInstance = true;
	}

	if (bShutDownExistingInstanceForDataPath)
	{
		if (!ShutDownZenServerProcessLockingDataDir(InSettings.DataPath))
		{
			PromptUserOfFailedShutDownOfExistingProcess(InSettings.DesiredPort);
			return false;
		}
	}

	if (ShutdownExistingInstanceForPid != 0)
	{
		if (!ShutdownZenServerProcess(ShutdownExistingInstanceForPid))
		{
			PromptUserOfFailedShutDownOfExistingProcess(InSettings.DesiredPort);
			return false;
		}
	}

	if (bLaunchNewInstance)
	{
		if (InSettings.bIsDefaultDataPath && InSettings.bIsDefaultSharedRunContext)
		{
			// See if the default data path is migrating, and if so, clean up after the old one.
			// Non-default data paths don't do the same thing because users are free to switch them back and forth
			// and expext the contents to remain when they change.  Only the default one cleans up after itself
			// to avoid a situation wherey the accumulate over time as the default location changes in config.
			// This cleanup is best-effort and may fail if an instance is unexpectedly still using the previous path.
			EnsureEditorSettingsConfigLoaded();
			FString InUseDefaultDataPath;
			if (!GConfig->GetString(TEXT("/Script/UnrealEd.ZenServerSettings"), TEXT("InUseDefaultDataPath"), InUseDefaultDataPath, GEditorSettingsIni))
			{
				InUseDefaultDataPath = FPaths::ConvertRelativePathToFull(FPaths::Combine(FPlatformProcess::ApplicationSettingsDir(), TEXT("Zen\\Data")));
			}
			if (!InUseDefaultDataPath.IsEmpty())
			{
				const FString InUseLockFilePath = FPaths::Combine(InUseDefaultDataPath, TEXT(".lock"));
				if (!FPaths::IsSamePath(InUseDefaultDataPath, InSettings.DataPath) && !IsZenProcessUsingDataDir(*InUseLockFilePath, nullptr))
				{
					UE_LOGF(LogZenServiceInstance, Display, "Migrating default data path from '%ls' to '%ls'.  Old location will be deleted.", *InUseDefaultDataPath, *InSettings.DataPath);
					IFileManager::Get().DeleteDirectory(*InUseDefaultDataPath, false, true);
				}
			}
		}

		FString ParmsWithoutTransients = DetermineCmdLineWithoutTransientComponents(InSettings.DataPath, InSettings.ExtraArgs, InSettings.Plugins, InSettings.bSendUnattendedBugReports, InSettings.RemoteNetworkService, InSettings.DesiredPort, SecurityConfigPath);

		if (InSettings.InstallMode == FServiceAutoLaunchSettings::EInstallMode::Link)
		{
			ParmsWithoutTransients.Appendf(TEXT(" --oidctoken-exe-path \"%s\""), *GetInTreeOidcTokenPath());
		}

		FZenLocalServiceRunContext EffectiveRunContext;
		EffectiveRunContext.Executable = ExecutablePath;
		EffectiveRunContext.CommandlineArguments = ParmsWithoutTransients;
		EffectiveRunContext.WorkingDirectory = WorkingDirectory;
		EffectiveRunContext.DataPath = InSettings.DataPath;
		EffectiveRunContext.bShowConsole = InSettings.bShowConsole;
		EffectiveRunContext.bLimitProcessLifetime = InSettings.bLimitProcessLifetime;

		if (StartLocalService(EffectiveRunContext))
		{
			// Only write run context if we're using the default shared run context
			if (InSettings.bIsDefaultSharedRunContext)
			{
				EffectiveRunContext.WriteToJsonFile(*ExecutionContextFilePath);
			}
		}
		else
		{
			UE_LOGF(LogZenServiceInstance, Warning, "Failed launch service using executable '%ls' on port %u", *ExecutablePath, InSettings.DesiredPort);
			return false;
		}
	}
	else if (InSettings.bLimitProcessLifetime)
	{
		ZenServerState State(/*ReadOnly*/ false);
		ZenServerState::ZenServerEntry* RunningEntry = State.LookupByDesiredListenPort(InSettings.DesiredPort);
		if (RunningEntry == nullptr)
		{
			UE_LOGF(LogZenServiceInstance, Warning, "Failed attach as sponsor process to executable '%ls' on port %u, can't find entry in shared state", *ExecutablePath, InSettings.DesiredPort);
		}
		else if (!RunningEntry->AddSponsorProcess(FPlatformProcess::GetCurrentProcessId()))
		{
			UE_LOGF(LogZenServiceInstance, Warning, "Failed attach as sponsor process to executable '%ls' on port %u, too many sponsored processes attached already or zenserver is unresponsive", *ExecutablePath, InSettings.DesiredPort);
		}
	}

	if (InSettings.bIsDefaultDataPath && InSettings.bIsDefaultSharedRunContext)
	{
		GConfig->SetString(TEXT("/Script/UnrealEd.ZenServerSettings"), TEXT("InUseDefaultDataPath"), *InSettings.DataPath, GEditorSettingsIni);
	}

	FStringView HostName = TEXT("[::1]");
	OutEndpoint = FZenServiceEndpoint(HostName, InSettings.DesiredPort);

	ZenLockFileData RunningLockFileState = ZenLockFileData::ReadCbLockFile(*LockFilePath);
	if (!RunningLockFileState.IsValid)
	{
		return false;
	}
	if (!RunningLockFileState.IsReady)
	{
		return false;
	}

	OutEndpoint = FZenServiceEndpoint(HostName, RunningLockFileState.EffectivePort);
	return true;
}

bool 
FZenServiceInstance::GetCacheStats(FZenCacheStats& Stats)
{
	{
		TUniqueLock Lock(LastCacheStatsMutex);
		// If we've already requested stats and they are ready then grab them
		if ( CacheStatsRequest.IsReady() == true )
		{
			LastCacheStats		= CacheStatsRequest.Get();
			LastCacheStatsTime	= FPlatformTime::Cycles64();

			CacheStatsRequest.Reset();
		}
	
		// Make a copy of the last updated stats
		Stats = LastCacheStats;

		const uint64 CurrentTime = FPlatformTime::Cycles64();
		constexpr double MinTimeBetweenRequestsInSeconds = 0.5;
		const double DeltaTimeInSeconds = FPlatformTime::ToSeconds64(CurrentTime - LastCacheStatsTime);

		if (CacheStatsRequest.IsValid() || DeltaTimeInSeconds <= MinTimeBetweenRequestsInSeconds)
		{
			return Stats.bIsValid;
		}
	}

#if WITH_EDITOR
	EAsyncExecution ThreadPool = EAsyncExecution::LargeThreadPool;
#else
	EAsyncExecution ThreadPool = EAsyncExecution::ThreadPool;
#endif
		// We've not got any requests in flight and we've met a given time requirement for requests
	CacheStatsRequest = Async(ThreadPool, [&Endpoint = Endpoint]
		{
			UE::Zen::FZenHttpRequest Request(Endpoint, false, DefaultSmallOperationConnectTimeoutMS, DefaultSmallOperationTimeoutMS);

			TArray64<uint8> GetBuffer;
			FZenHttpRequest::Result Result = Request.PerformBlockingDownload(TEXTVIEW("/stats/z$"), &GetBuffer, Zen::EContentType::CbObject);

			FZenCacheStats Stats;

			if (Result == Zen::FZenHttpRequest::Result::Success && Request.GetResponseCode() == 200)
			{
				FCbFieldView RootView(GetBuffer.GetData());
				Stats.bIsValid = LoadFromCompactBinary(RootView, Stats);
			}

			return Stats;
		});

	return Stats.bIsValid;
}

bool 
FZenServiceInstance::GetProjectStats(FZenProjectStats& Stats)
{
	{
		TUniqueLock Lock(LastProjectStatsMutex);
		// If we've already requested stats and they are ready then grab them
		if ( ProjectStatsRequest.IsReady() == true )
		{
			LastProjectStats		= ProjectStatsRequest.Get();
			LastProjectStatsTime	= FPlatformTime::Cycles64();

			ProjectStatsRequest.Reset();
		}
	
		// Make a copy of the last updated stats
		Stats = LastProjectStats;

		const uint64 CurrentTime = FPlatformTime::Cycles64();
		constexpr double MinTimeBetweenRequestsInSeconds = 0.5;
		const double DeltaTimeInSeconds = FPlatformTime::ToSeconds64(CurrentTime - LastProjectStatsTime);

		if (ProjectStatsRequest.IsValid() || DeltaTimeInSeconds <= MinTimeBetweenRequestsInSeconds)
		{
			return Stats.bIsValid;
		}
	}

#if WITH_EDITOR
	EAsyncExecution ThreadPool = EAsyncExecution::LargeThreadPool;
#else
	EAsyncExecution ThreadPool = EAsyncExecution::ThreadPool;
#endif
			// We've not got any requests in flight and we've met a given time requirement for requests
	ProjectStatsRequest = Async(ThreadPool, [&Endpoint = Endpoint]
		{
			UE::Zen::FZenHttpRequest Request(Endpoint, false, DefaultSmallOperationConnectTimeoutMS, DefaultSmallOperationTimeoutMS);

			TArray64<uint8> GetBuffer;
			FZenHttpRequest::Result Result = Request.PerformBlockingDownload(TEXTVIEW("/stats/prj"), &GetBuffer, Zen::EContentType::CbObject);

			FZenProjectStats Stats;

			if (Result == Zen::FZenHttpRequest::Result::Success && Request.GetResponseCode() == 200)
			{
				FCbFieldView RootView(GetBuffer.GetData());
				Stats.bIsValid = LoadFromCompactBinary(RootView, Stats);
			}

			return Stats;
		});

	return Stats.bIsValid;
}

bool 
FZenServiceInstance::GetGCStatus(FGCStatus& Status)
{
	check(IsInGameThread());

	// If we've already requested status and it is ready then grab it
	if (GCStatusRequest.IsReady() == true )
	{
		LastGCStatus	 = GCStatusRequest.Get();
		LastGCStatusTime = FPlatformTime::Cycles64();

		GCStatusRequest.Reset();
	}
	
	// Make a copy of the last updated status
	if (LastGCStatus.IsSet())
	{
		Status = LastGCStatus.GetValue();
	}

	const uint64 CurrentTime = FPlatformTime::Cycles64();
	constexpr double MinTimeBetweenRequestsInSeconds = 0.5;
	const double DeltaTimeInSeconds = FPlatformTime::ToSeconds64(CurrentTime - LastGCStatusTime);

	if (!GCStatusRequest.IsValid() && DeltaTimeInSeconds > MinTimeBetweenRequestsInSeconds)
	{
#if WITH_EDITOR
		EAsyncExecution ThreadPool = EAsyncExecution::LargeThreadPool;
#else
		EAsyncExecution ThreadPool = EAsyncExecution::ThreadPool;
#endif
		// We've not got any requests in flight and we've met a given time requirement for requests
		GCStatusRequest = Async(ThreadPool, [this]
			{
				UE::Zen::FZenHttpRequest Request(Endpoint, false, DefaultSmallOperationConnectTimeoutMS, DefaultSmallOperationTimeoutMS);

				TArray64<uint8> GetBuffer;
				FZenHttpRequest::Result Result = Request.PerformBlockingDownload(TEXTVIEW("/admin/gc"), &GetBuffer, Zen::EContentType::CbObject);

				TOptional<FGCStatus> GCStatus;

				if (Result == Zen::FZenHttpRequest::Result::Success && Request.GetResponseCode() == 200)
				{
					FCbObjectView RootObjectView(GetBuffer.GetData());

					GCStatus.Emplace();
					GCStatus->Description = FString(RootObjectView["Status"].AsString());
				}

				return GCStatus;
			});
	}

	return LastGCStatus.IsSet();
}

bool 
FZenServiceInstance::RequestGC(const bool* OverrideCollectSmallObjects, const uint32* OverrideMaxCacheDuration)
{
	UE::Zen::FZenHttpRequest Request(Endpoint, /*bLogErrors*/ false, DefaultSmallOperationConnectTimeoutMS, DefaultSmallOperationTimeoutMS);

	TCHAR Separators[] = {TEXT('?'), TEXT('&')};
	int32 SeparatorIndex = 0;
	TStringBuilder<128> Query;
	Query << TEXTVIEW("/admin/gc");

	if (OverrideCollectSmallObjects)
	{
		Query << Separators[SeparatorIndex] << TEXT("smallobjects=") << ::LexToString(*OverrideCollectSmallObjects);
		SeparatorIndex = FMath::Min(SeparatorIndex + 1, (int32)UE_ARRAY_COUNT(Separators));
	}

	if (OverrideMaxCacheDuration)
	{
		Query << Separators[SeparatorIndex] << TEXT("maxcacheduration=") << ::LexToString(*OverrideMaxCacheDuration);
		SeparatorIndex = FMath::Min(SeparatorIndex + 1, (int32)UE_ARRAY_COUNT(Separators));
	}

	FZenHttpRequest::Result Result = Request.PerformBlockingPost(Query, FMemoryView());

	if (Result == Zen::FZenHttpRequest::Result::Success && Request.GetResponseCode() == 200)
	{
		FCbObjectView ResponseObject = FCbObjectView(Request.GetResponseBuffer().GetData());
		FUtf8StringView ResponseStatus = ResponseObject["status"].AsString();

		return (ResponseStatus == "Started") || (ResponseStatus == "Running");
	}
	return false;
}

bool 
FZenServiceInstance::GatherAnalytics(TArray<FAnalyticsEventAttribute>& Attributes)
{
	FZenCacheStats ZenCacheStats;
	FZenProjectStats ZenProjectStats;

	if (GetCacheStats(ZenCacheStats) == false)
		return false;

	if (GetProjectStats(ZenProjectStats) == false)
		return false;

	const FString BaseName = TEXT("Zen_");

	{
		FString AttrName = BaseName + TEXT("Enabled");
		Attributes.Emplace(MoveTemp(AttrName), ZenCacheStats.bIsValid && ZenProjectStats.bIsValid);
	}

	///////////// Cache
	{
		FString AttrName = BaseName + TEXT("Cache_Size_Disk");
		Attributes.Emplace(MoveTemp(AttrName), ZenCacheStats.General.Size.Disk);
	}

	{
		FString AttrName = BaseName + TEXT("Cache_Size_Memory");
		Attributes.Emplace(MoveTemp(AttrName), ZenCacheStats.General.Size.Memory);
	}

	{
		FString AttrName = BaseName + TEXT("Cache_Hits");
		Attributes.Emplace(MoveTemp(AttrName), ZenCacheStats.General.Hits);
	}

	{
		FString AttrName = BaseName + TEXT("Cache_Misses");
		Attributes.Emplace(MoveTemp(AttrName), ZenCacheStats.General.Misses);
	}

	{
		FString AttrName = BaseName + TEXT("Cache_Writes");
		Attributes.Emplace(MoveTemp(AttrName), ZenCacheStats.General.Writes);
	}

	{
		FString AttrName = BaseName + TEXT("Cache_HitRatio");
		Attributes.Emplace(MoveTemp(AttrName), ZenCacheStats.General.HitRatio);
	}

	{
		FString AttrName = BaseName + TEXT("Cache_Cas_Hits");
		Attributes.Emplace(MoveTemp(AttrName), ZenCacheStats.General.CidHits);
	}

	{
		FString AttrName = BaseName + TEXT("Cache_Cas_Misses");
		Attributes.Emplace(MoveTemp(AttrName), ZenCacheStats.General.CidMisses);
	}

	{
		FString AttrName = BaseName + TEXT("Cache_Cas_Writes");
		Attributes.Emplace(MoveTemp(AttrName), ZenCacheStats.General.CidWrites);
	}

	{
		FString AttrName = BaseName + TEXT("Cache_Requests");
		Attributes.Emplace(MoveTemp(AttrName), ZenCacheStats.Request.Count);
	}

	{
		FString AttrName = BaseName + TEXT("Cache_BadRequests");
		Attributes.Emplace(MoveTemp(AttrName), ZenCacheStats.General.BadRequestCount);
	}


	{
		FString AttrName = BaseName + TEXT("Cache_Requests_Count");
		Attributes.Emplace(MoveTemp(AttrName), ZenCacheStats.Request.Count);
	}

	{
		FString AttrName = BaseName + TEXT("Cache_Requests_RateMean");
		Attributes.Emplace(MoveTemp(AttrName), ZenCacheStats.Request.RateMean);
	}

	{
		FString AttrName = BaseName + TEXT("Cache_Requests_TAverage");
		Attributes.Emplace(MoveTemp(AttrName), ZenCacheStats.Request.TAverage);
	}

	{
		FString AttrName = BaseName + TEXT("Cache_Requests_TMin");
		Attributes.Emplace(MoveTemp(AttrName), ZenCacheStats.Request.TMin);
	}

	{
		FString AttrName = BaseName + TEXT("Cache_Requests_TMax");
		Attributes.Emplace(MoveTemp(AttrName), ZenCacheStats.Request.TMax);
	}

	{
		FString AttrName = BaseName + TEXT("Cache_TotalUploadedMB");
		Attributes.Emplace(MoveTemp(AttrName), ZenCacheStats.Upstream.TotalUploadedMB);
	}

	{
		FString AttrName = BaseName + TEXT("Upstream_TotalDownloadedMB");
		Attributes.Emplace(MoveTemp(AttrName), ZenCacheStats.Upstream.TotalDownloadedMB);
	}

	{
		FString AttrName = BaseName + TEXT("Upstream_TotalUploadedMB");
		Attributes.Emplace(MoveTemp(AttrName), ZenCacheStats.Upstream.TotalUploadedMB);
	}

	{
		FString AttrName = BaseName + TEXT("Upstream_Requests_Count");
		Attributes.Emplace(MoveTemp(AttrName), ZenCacheStats.UpstreamRequest.Count);
	}
	{
		FString AttrName = BaseName + TEXT("Upstream_Requests_RateMean");
		Attributes.Emplace(MoveTemp(AttrName), ZenCacheStats.UpstreamRequest.RateMean);
	}
	{
		FString AttrName = BaseName + TEXT("Upstream_Requests_TAverage");
		Attributes.Emplace(MoveTemp(AttrName), ZenCacheStats.UpstreamRequest.TAverage);
	}
	{
		FString AttrName = BaseName + TEXT("Upstream_Requests_TMin");
		Attributes.Emplace(MoveTemp(AttrName), ZenCacheStats.UpstreamRequest.TMin);
	}
	{
		FString AttrName = BaseName + TEXT("Upstream_Requests_TMax");
		Attributes.Emplace(MoveTemp(AttrName), ZenCacheStats.UpstreamRequest.TMax);
	}

	{
		FString AttrName = BaseName + TEXT("Cas_Size_Large");
		Attributes.Emplace(MoveTemp(AttrName), ZenCacheStats.CID.Size.Large);
	}

	{
		FString AttrName = BaseName + TEXT("Cas_Size_Small");
		Attributes.Emplace(MoveTemp(AttrName), ZenCacheStats.CID.Size.Small);
	}

	{
		FString AttrName = BaseName + TEXT("Cas_Size_Tiny");
		Attributes.Emplace(MoveTemp(AttrName), ZenCacheStats.CID.Size.Tiny);
	}

	{
		FString AttrName = BaseName + TEXT("Cas_Size_Total");
		Attributes.Emplace(MoveTemp(AttrName), ZenCacheStats.CID.Size.Total);
	}

	///////////// Project
	{
		FString AttrName = BaseName + TEXT("Project_Size_Disk");
		Attributes.Emplace(MoveTemp(AttrName), ZenProjectStats.General.Size.Disk);
	}

	{
		FString AttrName = BaseName + TEXT("Project_Size_Memory");
		Attributes.Emplace(MoveTemp(AttrName), ZenProjectStats.General.Size.Memory);
	}

	{
		FString AttrName = BaseName + TEXT("Project_WriteCount");
		Attributes.Emplace(MoveTemp(AttrName), ZenProjectStats.General.Project.WriteCount);
	}

	{
		FString AttrName = BaseName + TEXT("Project_ReadCount");
		Attributes.Emplace(MoveTemp(AttrName), ZenProjectStats.General.Project.ReadCount);
	}

	{
		FString AttrName = BaseName + TEXT("Project_DeleteCount");
		Attributes.Emplace(MoveTemp(AttrName), ZenProjectStats.General.Project.DeleteCount);
	}

	{
		FString AttrName = BaseName + TEXT("Project_Oplog_WriteCount");
		Attributes.Emplace(MoveTemp(AttrName), ZenProjectStats.General.Oplog.WriteCount);
	}

	{
		FString AttrName = BaseName + TEXT("Project_Oplog_ReadCount");
		Attributes.Emplace(MoveTemp(AttrName), ZenProjectStats.General.Oplog.ReadCount);
	}

	{
		FString AttrName = BaseName + TEXT("Project_Oplog_DeleteCount");
		Attributes.Emplace(MoveTemp(AttrName), ZenProjectStats.General.Oplog.DeleteCount);
	}

	{
		FString AttrName = BaseName + TEXT("Project_Op_Hits");
		Attributes.Emplace(MoveTemp(AttrName), ZenProjectStats.General.Op.HitCount);
	}

	{
		FString AttrName = BaseName + TEXT("Project_Op_Misses");
		Attributes.Emplace(MoveTemp(AttrName), ZenProjectStats.General.Op.MissCount);
	}

	{
		FString AttrName = BaseName + TEXT("Project_Op_Writes");
		Attributes.Emplace(MoveTemp(AttrName), ZenProjectStats.General.Op.WriteCount);
	}

	{
		FString AttrName = BaseName + TEXT("Project_Chunk_Hits");
		Attributes.Emplace(MoveTemp(AttrName), ZenProjectStats.General.Chunk.HitCount);
	}

	{
		FString AttrName = BaseName + TEXT("Project_Chunk_Misses");
		Attributes.Emplace(MoveTemp(AttrName), ZenProjectStats.General.Chunk.MissCount);
	}

	{
		FString AttrName = BaseName + TEXT("Project_Chunk_Writes");
		Attributes.Emplace(MoveTemp(AttrName), ZenProjectStats.General.Chunk.WriteCount);
	}

	{
		FString AttrName = BaseName + TEXT("Project_Requests");
		Attributes.Emplace(MoveTemp(AttrName), ZenProjectStats.General.RequestCount);
	}

	{
		FString AttrName = BaseName + TEXT("Project_BadRequests");
		Attributes.Emplace(MoveTemp(AttrName), ZenProjectStats.General.BadRequestCount);
	}

	{
		FString AttrName = BaseName + TEXT("Project_Op_HitRatio");
		double Total = static_cast<double>(ZenProjectStats.General.Op.HitCount + ZenProjectStats.General.Op.MissCount);
		Attributes.Emplace(MoveTemp(AttrName), Total > 0 ? static_cast<double>(ZenProjectStats.General.Op.HitCount) / Total : 0.0);
	}

	{
		FString AttrName = BaseName + TEXT("Project_Chunk_HitRatio");
		double Total = static_cast<double>(ZenProjectStats.General.Chunk.HitCount + ZenProjectStats.General.Chunk.MissCount);
		Attributes.Emplace(MoveTemp(AttrName), Total > 0 ? static_cast<double>(ZenProjectStats.General.Chunk.HitCount) / Total : 0.0);
	}

	return true;
}

bool
FZenServiceInstance::GetWorkspaces(FZenWorkspaces& Workspaces) const
{
#if WITH_EDITOR
	EAsyncExecution ThreadPool = EAsyncExecution::LargeThreadPool;
#else
	EAsyncExecution ThreadPool = EAsyncExecution::ThreadPool;
#endif

	WorkspacesRequest = Async(ThreadPool, [&Endpoint = Endpoint]
	{
		UE::Zen::FZenHttpRequest Request(Endpoint, false, DefaultSmallOperationConnectTimeoutMS, DefaultSmallOperationTimeoutMS);

		FZenHttpRequest::Result Result = Request.PerformBlockingDownload(TEXTVIEW("/ws"), nullptr, Zen::EContentType::CbObject);

		FZenWorkspaces Workspaces;

		if (Result == Zen::FZenHttpRequest::Result::Success && Request.GetResponseCode() == 200)
		{
			FMemoryReaderView Reader(Request.GetResponseBuffer());
			FCbObject ResponseObj = LoadCompactBinary(Reader).AsObject();

			FCbArrayView WorkspacesArray = ResponseObj["workspaces"].AsArrayView();
			for (FCbFieldView WorkspaceField : WorkspacesArray)
			{
				FCbObjectView WorkspaceObject = WorkspaceField.AsObjectView();

				FZenWorkspaces::Workspace Workspace = {
					.Id = *WriteToString<64>(WorkspaceObject["id"].AsObjectId()),
					.BaseDir = FString(WorkspaceObject["root_path"].AsString()),
					.bDynamicShare = WorkspaceObject["allow_share_creation_from_http"].AsBool()
				};

				if (!Workspace.Id.IsEmpty())
				{
					FCbArrayView WorkspaceShares = WorkspaceObject["shares"].AsArrayView();

					UE_LOGF(LogZenServiceInstance, Warning, "Number of shares %lld", WorkspaceShares.Num());

					for (int32 Idx = 0; Idx < WorkspaceShares.Num(); ++Idx)
					{
						UE_LOGF(LogZenServiceInstance, Warning, "Iterating through share %d", Idx);
					}

					for (FCbFieldView ShareField : WorkspaceShares)
					{
						FCbObjectView ShareObject = ShareField.AsObjectView();

						FZenWorkspaces::Share Share = {
							.Id = *WriteToString<64>(ShareObject["id"].AsObjectId()),
							.Dir = FString(ShareObject["share_path"].AsString()),
							.Alias = FString(ShareObject["alias"].AsString())
						};

						if (!Share.Id.IsEmpty())
						{
							Workspace.WorkspaceShares.Add(Share);
						}
					}

					Workspaces.ZenWorkspaces.Add(Workspace);
				}
			}

			Workspaces.bIsValid = true;
		}

		return Workspaces;
	});

	WorkspacesRequest.Wait();

	Workspaces = WorkspacesRequest.Get();
	return Workspaces.bIsValid;
}

int32
FZenServiceInstance::GetWorkspaceCount() const
{
	FZenWorkspaces Workspaces;
	if (!GetWorkspaces(Workspaces))
	{
		return 0;
	}

	return Workspaces.ZenWorkspaces.Num();
}

}
