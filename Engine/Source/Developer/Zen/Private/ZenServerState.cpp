// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZenServerState.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "Logging/LogMacros.h"
#include "Misc/ScopeExit.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Serialization/CompactBinaryValidation.h"

#if PLATFORM_WINDOWS
#	include "Windows/AllowWindowsPlatformTypes.h"
#	include <shellapi.h>
#	include <synchapi.h>
#	include <Windows.h>
#	include "Windows/HideWindowsPlatformTypes.h"
#endif

#if PLATFORM_UNIX || PLATFORM_MAC
#	include <sys/file.h>
#	include <sys/mman.h>
#	include <sys/sem.h>
#	include <sys/stat.h>
#	include <sys/sysctl.h>
#	include <signal.h>
#	include <unistd.h>
#endif

DEFINE_LOG_CATEGORY_STATIC(LogZenServiceState, Log, All);

#if PLATFORM_UNIX
static char
GetPidStatus(int Pid)
{
	TAnsiStringBuilder<128> StatPath;
	StatPath.Appendf("/proc/%d/stat", Pid);
	FILE* StatFile = fopen(*StatPath, "r");
	if (StatFile)
	{
		char Buffer[5120];
		int	 Size = fread(Buffer, 1, 5120 - 1, StatFile);
		fclose(StatFile);
		if (Size > 0)
		{
			Buffer[Size] = 0;
			char* ScanPtr = strrchr(Buffer, ')');
			if (ScanPtr && ScanPtr[1] != '\0')
			{
				ScanPtr += 2;
				char State = *ScanPtr;
				return State;
			}
		}
	}
	return 0;
}

static bool
IsZombieProcess(int pid)
{
	char Status = GetPidStatus(pid);
	if (Status == 'Z' || Status == 0)
	{
		return true;
	}
	return false;
}

#endif	// ZEN_PLATFORM_LINUX

#if PLATFORM_MAC
static bool
IsZombieProcess(int pid)
{
	struct kinfo_proc Info;
	int				  Mib[4] = { CTL_KERN, KERN_PROC, KERN_PROC_PID, pid };
	size_t			  InfoSize = sizeof Info;

	int Res = sysctl(Mib, 4, &Info, &InfoSize, NULL, 0);
	if (Res != 0)
	{
		return false;
	}
	if (Info.kp_proc.p_stat == SZOMB)
	{
		// Zombie process
		return true;
	}
	return false;
}
#endif	// PLATFORM_MAC

// Native functions to interact with a process using a process id
// We don't use UE's own OpenProcess as they try to open processes with PROCESS_ALL_ACCESS
bool ZenServerState::IsProcessRunning(uint32 Pid)
{
	if (Pid == 0)
	{
		return false;
	}
#if PLATFORM_WINDOWS
	HANDLE Handle = OpenProcess(PROCESS_QUERY_LIMITED_INFORMATION, 0, (DWORD)Pid);

	if (!Handle)
	{
		DWORD Error = GetLastError();

		if (Error == ERROR_INVALID_PARAMETER)
		{
			return false;
		}
		else if (Error == ERROR_ACCESS_DENIED)
		{
			// This occurs when running as a system service, as we're not the user that started the process.
			return true;
		}
		UE_LOGF(LogZenServiceState, Warning, "Failed to open running process %d: %d, assuming it is not running", Pid, Error);
		return false;
	}
	ON_SCOPE_EXIT{ CloseHandle(Handle); };

	DWORD ExitCode = 0;
	if (GetExitCodeProcess(Handle, &ExitCode) == 0)
	{
		DWORD Error = GetLastError();
		UE_LOGF(LogZenServiceState, Warning, "Failed to get running process exit code %d: %d, assuming it is still running", Pid, Error);
		return true;
	}
	else if (ExitCode == STILL_ACTIVE)
	{
		return true;
	}

	return false;

#elif PLATFORM_UNIX || PLATFORM_MAC
	int Res = kill(pid_t(Pid), 0);
	if (Res == 0)
	{
		if (IsZombieProcess(Pid))
		{
			return false;
		}
		return true;
	}
	int Error = errno;
	if (Error == EPERM)
	{
		UE_LOGF(LogZenServiceState, Warning, "No permission to signal running process %d: %d, assuming it is running", Pid, Error);
		return true;
	}
	else if (Error == ESRCH)
	{
		return false;
	}
	UE_LOGF(LogZenServiceState, Warning, "Failed to signal running process %d: %d, assuming it is running", Pid, Error);
	return true;
#endif
}

bool ZenServerState::Terminate(uint32 Pid)
{
#if PLATFORM_WINDOWS
	HANDLE Handle = OpenProcess(PROCESS_TERMINATE | SYNCHRONIZE, 0, (DWORD)Pid);
	if (Handle == NULL)
	{
		DWORD Error = GetLastError();

		if (Error != ERROR_INVALID_PARAMETER)
		{
			UE_LOGF(LogZenServiceState, Warning, "Failed to open running process for terminate %d: %d", Pid, Error);
			return false;
		}
		return true;
	}
	ON_SCOPE_EXIT{ CloseHandle(Handle); };

	BOOL bTerminated = TerminateProcess(Handle, 0);
	if (!bTerminated)
	{
		DWORD Error = GetLastError();
		UE_LOGF(LogZenServiceState, Warning, "Failed to terminate running process %d: %d", Pid, Error);
		return false;
	}
	DWORD WaitResult = WaitForSingleObject(Handle, 15000);
	BOOL bSuccess = (WaitResult == WAIT_OBJECT_0) || (WaitResult == WAIT_ABANDONED_0);
	if (!bSuccess)
	{
		if (WaitResult == WAIT_FAILED)
		{
			DWORD Error = GetLastError();
			UE_LOGF(LogZenServiceState, Warning, "Failed to wait for terminated process %d: %d", Pid, Error);
		}
		return false;
	}
#elif PLATFORM_UNIX || PLATFORM_MAC
	int Res = kill(pid_t(Pid), SIGKILL);
	if (Res != 0)
	{
		int LastError = errno;
		if (LastError != ESRCH)
		{
			UE_LOGF(LogZenServiceState, Warning, "Failed to terminate running process %d: %d", Pid, LastError);
			return false;
		}
	}
#endif
	return true;
}

bool ZenServerState::FindRunningProcessId(const TCHAR* ExecutablePath, uint32* OutPid)
{
	FString NormalizedExecutablePath(ExecutablePath);
	FPaths::NormalizeFilename(NormalizedExecutablePath);
	FPlatformProcess::FProcEnumerator ProcIter;
	while (ProcIter.MoveNext())
	{
		FPlatformProcess::FProcEnumInfo ProcInfo = ProcIter.GetCurrent();
		FString Candidate = ProcInfo.GetFullPath();
		FPaths::NormalizeFilename(Candidate);
		if (Candidate == NormalizedExecutablePath)
		{
			if (OutPid)
			{
				*OutPid = ProcInfo.GetPID();
			}
			return true;
		}
	}
	return false;
}

ZenServerState::ZenServerState(bool ReadOnly)
	: m_hMapFile(nullptr)
	, m_Data(nullptr)
{
	size_t MapSize = m_MaxEntryCount * sizeof(ZenServerEntry);

#if PLATFORM_WINDOWS
	DWORD DesiredAccess = ReadOnly ? FILE_MAP_READ : (FILE_MAP_READ | FILE_MAP_WRITE);
	HANDLE hMap = OpenFileMapping(DesiredAccess, 0, L"Global\\ZenMap");
	if (hMap == NULL)
	{
		hMap = OpenFileMapping(DesiredAccess, 0, L"Local\\ZenMap");
	}

	if (hMap == NULL)
	{
		return;
	}

	void* pBuf = MapViewOfFile(hMap,		   // handle to map object
		DesiredAccess,  // read permission
		0,			   // offset high
		0,			   // offset low
		MapSize);

	if (pBuf == NULL)
	{
		CloseHandle(hMap);
		return;
	}
#elif PLATFORM_UNIX || PLATFORM_MAC
	int OFlag = ReadOnly ? (O_RDONLY) : (O_RDWR | O_CREAT);
	int Fd = shm_open("/UnrealEngineZen", OFlag, 0666);
	if (Fd < 0)
	{
		return;
	}
	void* hMap = (void*)intptr_t(Fd);

	int Prot = ReadOnly ? PROT_READ : (PROT_WRITE | PROT_READ);
	void* pBuf = mmap(nullptr, MapSize, Prot, MAP_SHARED, Fd, 0);
	if (pBuf == MAP_FAILED)
	{
		close(Fd);
		return;
	}
#endif

#if PLATFORM_WINDOWS || PLATFORM_UNIX || PLATFORM_MAC
	m_hMapFile = hMap;
	m_Data = reinterpret_cast<ZenServerEntry*>(pBuf);
#endif
	m_IsReadOnly = ReadOnly;
}

ZenServerState::~ZenServerState()
{
#if PLATFORM_WINDOWS
	if (m_Data)
	{
		UnmapViewOfFile(m_Data);
	}

	if (m_hMapFile)
	{
		CloseHandle(m_hMapFile);
	}
#elif PLATFORM_UNIX || PLATFORM_MAC
	if (m_Data != nullptr)
	{
		munmap((void*)m_Data, m_MaxEntryCount * sizeof(ZenServerEntry));
	}

	int Fd = int(intptr_t(m_hMapFile));
	close(Fd);
#endif
	m_hMapFile = nullptr;
	m_Data = nullptr;
}

const ZenServerState::ZenServerEntry* ZenServerState::LookupByDesiredListenPortInternal(int Port) const
{
	if (m_Data == nullptr)
	{
		return nullptr;
	}

	for (int i = 0; i < m_MaxEntryCount; ++i)
	{
		if (m_Data[i].DesiredListenPort.load(std::memory_order_relaxed) == Port)
		{
			const ZenServerState::ZenServerEntry* Entry = &m_Data[i];
			if (IsProcessRunning((uint32)Entry->Pid.load(std::memory_order_relaxed)))
			{
				return Entry;
			}
		}
	}

	return nullptr;
}

const ZenServerState::ZenServerEntry* ZenServerState::LookupByDesiredListenPort(int Port) const
{
	return LookupByDesiredListenPortInternal(Port);
}

ZenServerState::ZenServerEntry* ZenServerState::LookupByDesiredListenPort(int Port)
{
	check(!m_IsReadOnly);
	return const_cast<ZenServerState::ZenServerEntry*>(LookupByDesiredListenPortInternal(Port));
}

const ZenServerState::ZenServerEntry* ZenServerState::LookupByEffectiveListenPortInternal(int Port) const
{
	if (m_Data == nullptr)
	{
		return nullptr;
	}

	for (int i = 0; i < m_MaxEntryCount; ++i)
	{
		const ZenServerState::ZenServerEntry* Entry = &m_Data[i];
		if (Entry->EffectiveListenPort.load(std::memory_order_relaxed) == Port)
		{
			if (IsProcessRunning((uint32)Entry->Pid.load(std::memory_order_relaxed)))
			{
				return Entry;
			}
		}
	}

	return nullptr;
}

const ZenServerState::ZenServerEntry* ZenServerState::LookupByEffectiveListenPort(int Port) const
{
	return LookupByEffectiveListenPortInternal(Port);
}

ZenServerState::ZenServerEntry* ZenServerState::LookupByEffectiveListenPort(int Port)
{
	check(!m_IsReadOnly);
	return const_cast<ZenServerState::ZenServerEntry*>(LookupByEffectiveListenPortInternal(Port));
}

const ZenServerState::ZenServerEntry* ZenServerState::LookupByPid(uint32 Pid) const
{
	if (m_Data == nullptr)
	{
		return nullptr;
	}

	for (int i = 0; i < m_MaxEntryCount; ++i)
	{
		const ZenServerState::ZenServerEntry* Entry = &m_Data[i];
		if (m_Data[i].Pid.load(std::memory_order_relaxed) == Pid)
		{
			if (IsProcessRunning(Pid))
			{
				return Entry;
			}
		}
	}

	return nullptr;
}

bool
ZenServerState::ZenServerEntry::AddSponsorProcess(uint32 PidToAdd)
{
	uint32_t ServerPid = Pid.load();

	// Sponsor processes are checked every second, so 2 second wait time should be enough
	const FTimespan MaximumWaitForPickup = FTimespan::FromSeconds(2);
	auto	 WaitForPickup = [&](uint32_t AddedSlotIndex) {
		FDateTime StartedWaitingForPickup = FDateTime::UtcNow();
		while (SponsorPids[AddedSlotIndex] == PidToAdd)
		{
			FTimespan WaitForPickup = FDateTime::UtcNow() - StartedWaitingForPickup;
			if (WaitForPickup > MaximumWaitForPickup)
			{
				SponsorPids[AddedSlotIndex].compare_exchange_strong(PidToAdd, 0);
				return false;
			}
			if (!IsProcessRunning(ServerPid))
			{
				SponsorPids[AddedSlotIndex].compare_exchange_strong(PidToAdd, 0);
				return false;
			}
			FPlatformProcess::Sleep(0.1f);
		}
		return true;
	};
	for (uint32_t SponsorIndex = 0; SponsorIndex < 8; SponsorIndex++)
	{
		if (SponsorPids[SponsorIndex].load(std::memory_order_relaxed) == PidToAdd)
		{
			return WaitForPickup(SponsorIndex);
		}
		uint32_t Expected = 0;
		if (SponsorPids[SponsorIndex].compare_exchange_strong(Expected, PidToAdd))
		{
			return WaitForPickup(SponsorIndex);
		}
	}
	return false;
}

ZenSharedEvent::ZenSharedEvent(FStringView EventName)
	: m_EventName(EventName)
{
	check(m_EventName.Len() > 0);
}
ZenSharedEvent::~ZenSharedEvent()
{
	Close();
}

bool ZenSharedEvent::Create()
{
#if PLATFORM_WINDOWS
	check(m_EventHandle == NULL);
	FString FullEventName = GetFullEventName();
	m_EventHandle = (void*)CreateEventW(nullptr, true, false, *FullEventName);
	if (m_EventHandle == NULL)
	{
		DWORD LastError = GetLastError();
		UE_LOGF(LogZenServiceState, Warning, "Failed creating named event '%ls' (err: %d)", *FullEventName, LastError);
		return false;
	}
	return true;
#elif PLATFORM_UNIX || PLATFORM_MAC
	check(m_Fd == -1);
	check(m_Semaphore == -1);
	FAnsiString EventPath = GetEventPath();
	// Create a file to back the semaphore
	m_Fd = open(*EventPath, O_RDWR | O_CREAT | O_CLOEXEC, 0666);
	if (m_Fd < 0)
	{
		int LastError = errno;
		UE_LOGF(LogZenServiceState, Warning, "Failed to create named event '%s' (err: %d)", *EventPath, LastError);
		return false;
	}
	fchmod(m_Fd, 0666);

	// Use the file path to generate an IPC key
	key_t IpcKey = ftok(*EventPath, 1);
	if (IpcKey < 0)
	{
		int LastError = errno;
		close(m_Fd);
		m_Fd = -1;
		UE_LOGF(LogZenServiceState, Warning, "Failed to create an SysV IPC key for named event '%s' (err: %d)", *EventPath, LastError);
		return false;
	}

	// Use the key to create/open the semaphore
	m_Semaphore = semget(IpcKey, 1, 0600 | IPC_CREAT);
	if (m_Semaphore < 0)
	{
		int LastError = errno;
		close(m_Fd);
		m_Fd = -1;
		UE_LOGF(LogZenServiceState, Warning, "Failed creating an SysV semaphore for named event '%s' (err: %d)", *EventPath, LastError);
		return false;
	}

	// Atomically claim ownership of the semaphore's key. The owner initializes
	// the semaphore to 1 so we can use the wait-for-zero op as that does not
	// modify the semaphore's value on a successful wait.
	int LockResult = flock(m_Fd, LOCK_EX | LOCK_NB);
	if (LockResult == 0)
	{
		// This isn't thread safe really. Another thread could open the same
		// semaphore and successfully wait on it in the period of time where
		// this comment is but before the semaphore's initialised.
		semctl(m_Semaphore, 0, SETVAL, 1);
	}
	return true;
#endif
}

bool ZenSharedEvent::Exists() const
{
#if PLATFORM_WINDOWS
	FString FullEventName = GetFullEventName();
	HANDLE EventHandle = OpenEventW(READ_CONTROL, false, *FullEventName);
	if (EventHandle == NULL)
	{
		DWORD LastError = GetLastError();
		if (LastError != ERROR_FILE_NOT_FOUND)
		{
			UE_LOGF(LogZenServiceState, Warning, "Failed checking existance of named event '%ls' (err: %d)", *FullEventName, LastError);
		}
		return false;
	}
	else
	{
		CloseHandle(EventHandle);
		return false;
	}
#elif PLATFORM_UNIX || PLATFORM_MAC
	FAnsiString EventPath = GetEventPath();
	key_t IpcKey = ftok(*EventPath, 1);
	if (IpcKey < 0)
	{
		int LastError = errno;
		UE_LOGF(LogZenServiceState, Warning, "Failed to create an SysV IPC key for named event '%s' (err: %d)", *EventPath, LastError);
		return false;
	}
	int Semaphore = semget(IpcKey, 1, 0400);
	if (Semaphore < 0)
	{
		int LastError = errno;
		if (LastError != ENOENT)
		{
			UE_LOGF(LogZenServiceState, Warning, "Failed checking named event '%s' (err: %d)", *EventPath, LastError);
		}
		return false;
	}
	return true;
#endif
}

bool ZenSharedEvent::Open()
{
#if PLATFORM_WINDOWS
	check(m_EventHandle == NULL);
	FString FullEventName = GetFullEventName();
	m_EventHandle = (void*)OpenEventW(EVENT_MODIFY_STATE, false, *FullEventName);
	if (m_EventHandle == NULL)
	{
		DWORD LastError = GetLastError();
		if (LastError != ERROR_FILE_NOT_FOUND)
		{
			UE_LOGF(LogZenServiceState, Warning, "Failed opening named event '%ls' (err: %d)", *FullEventName, LastError);
		}
		return false;
	}
	return true;
#elif PLATFORM_UNIX || PLATFORM_MAC
	check(m_Fd == -1);
	check(m_Semaphore == -1);
	FAnsiString EventPath = GetEventPath();
	key_t IpcKey = ftok(*EventPath, 1);
	if (IpcKey < 0)
	{
		int LastError = errno;
		UE_LOGF(LogZenServiceState, Warning, "Failed to create an SysV IPC key for named event '%s' (err: %d)", *EventPath, LastError);
		return false;
	}
	m_Semaphore = semget(IpcKey, 1, 0600);
	if (m_Semaphore < 0)
	{
		int LastError = errno;
		if (LastError != ENOENT)
		{
			UE_LOGF(LogZenServiceState, Warning, "Failed opening named event '%s' (err: %d)", *EventPath, LastError);
		}
		return false;
	}
	return true;
#endif
}

bool ZenSharedEvent::Wait(int TimeoutMs)
{
#if PLATFORM_WINDOWS
	check(m_EventHandle != NULL);

	const DWORD Timeout = (TimeoutMs < 0) ? INFINITE : TimeoutMs;

	DWORD Result = WaitForSingleObject((HANDLE)m_EventHandle, Timeout);

	if (Result == WAIT_FAILED)
	{
		DWORD LastError = GetLastError();
		UE_LOGF(LogZenServiceState, Warning, "Failed waiting for named event '%ls' (err: %d)", *m_EventName, LastError);
		return false;
	}

	return (Result == WAIT_OBJECT_0) || (Result == WAIT_ABANDONED_0);

#elif PLATFORM_UNIX || PLATFORM_MAC
	check(m_Semaphore != -1);

	int			  Result;
	struct sembuf SemOp = {};

	if (TimeoutMs < 0)
	{
		Result = semop(m_Semaphore, &SemOp, 1);
		if (Result != 0)
		{
			int LastError = errno;
			UE_LOGF(LogZenServiceState, Warning, "Failed waiting for named event '%ls' (err: %d)", *m_EventName, LastError);
			return false;
		}
		return true;
	}
#if PLATFORM_UNIX
	const int		TimeoutSec = TimeoutMs / 1000;
	struct timespec TimeoutValue = {
		.tv_sec = TimeoutSec,
		.tv_nsec = (TimeoutMs - (TimeoutSec * 1000)) * 1000000,
	};
	Result = semtimedop(m_Semaphore, &SemOp, 1, &TimeoutValue);
	if (Result == 0)
	{
		return true;
	}
	int LastError = errno;
	if (LastError != EAGAIN)
	{
		UE_LOGF(LogZenServiceState, Warning, "Failed waiting for named event '%ls' (err: %d)", *m_EventName, LastError);
	}
	return false;
#elif PLATFORM_MAC
	const unsigned int SleepTimeMs = 10u;
	SemOp.sem_flg = IPC_NOWAIT;
	do
	{
		Result = semop(m_Semaphore, &SemOp, 1);
		if (Result == 0)
		{
			return true;
		}
		else
		{
			int LastError = errno;
			if (errno != EAGAIN)
			{
				UE_LOGF(LogZenServiceState, Warning, "Failed waiting for named event '%ls' (err: %d)", *m_EventName, LastError);
				break;
			}
		}
		usleep(SleepTimeMs * 1000u);
		TimeoutMs -= SleepTimeMs;
	} while (TimeoutMs > 0);
	return false;
#endif // PLATFORM_MAC
#endif
}

bool ZenSharedEvent::Set()
{
#if PLATFORM_WINDOWS
	check(m_EventHandle != nullptr);
	if (SetEvent((HANDLE)m_EventHandle))
	{
		return true;
	}
	else
	{
		DWORD LastError = GetLastError();
		UE_LOGF(LogZenServiceState, Warning, "Failed signalling named event '%ls' (err: %d)", *m_EventName, LastError);
		return false;
	}
#elif PLATFORM_UNIX || PLATFORM_MAC
	check(m_Semaphore != -1);
	if (semctl(m_Semaphore, 0, SETVAL, 0) != -1)
	{
		return true;
	}
	else
	{
		int LastError = errno;
		UE_LOGF(LogZenServiceState, Warning, "Failed signalling named event '%ls' (err: %d)", *m_EventName, LastError);
		return false;
	}
#endif
}

void ZenSharedEvent::Close()
{
#if PLATFORM_WINDOWS
	if (m_EventHandle != NULL)
	{
		CloseHandle((HANDLE)m_EventHandle);
	}
	m_EventHandle = NULL;
#elif PLATFORM_UNIX || PLATFORM_MAC
	if (m_Fd != -1)
	{
		if (flock(m_Fd, LOCK_EX | LOCK_NB) == 0)
		{
			unlink(*m_EventPath);

			flock(m_Fd, LOCK_UN | LOCK_NB);
			close(m_Fd);

			semctl(m_Semaphore, 0, IPC_RMID);
		}
	}
	m_Fd = -1;
	m_Semaphore = -1;
#endif
}

#if PLATFORM_WINDOWS

FString ZenSharedEvent::GetFullEventName() const
{
	TStringBuilder<128> Name;
	Name << "Local\\";
	Name << m_EventName;
	return *Name;
}

#elif PLATFORM_UNIX || PLATFORM_MAC

FAnsiString ZenSharedEvent::GetEventPath() const
{
	TAnsiStringBuilder<128> Name;
	Name << "/tmp/";
	Name << m_EventName;
	return *Name;
}
#endif

FString ZenSharedEvent::GetShutdownEventName(uint16 EffectiveListenPort)
{
	return *WriteToWideString<64>(WIDETEXT("Zen_"), EffectiveListenPort, WIDETEXT("_Shutdown"));
}

FString ZenSharedEvent::GetStartupEventName()
{
	return (*WriteToWideString<64>(WIDETEXT("Zen_"), FPlatformProcess::GetCurrentProcessId(), WIDETEXT("_Startup")));
}



static ZenLockFileData ReadLockData(FUniqueBuffer&& FileBytes)
{
	if (ValidateCompactBinary(FileBytes, ECbValidateMode::Default) == ECbValidateError::None)
	{
		FCbObject LockObject(FileBytes.MoveToShared());

		int32 ProcessId = LockObject["pid"].AsInt32();
		FCbObjectId SessionId = LockObject["session_id"].AsObjectId();
		int32 EffectivePort = LockObject["port"].AsInt32();
		bool IsReady = LockObject["ready"].AsBool();
		FUtf8StringView DataDir = LockObject["data"].AsString();
		FUtf8StringView ExecutablePath = LockObject["executable"].AsString();

		bool IsValid = ProcessId > 0 && EffectivePort > 0 && EffectivePort <= 0xffff;

		return ZenLockFileData{
			.ProcessId  = static_cast<uint32>(ProcessId),
			.SessionId = SessionId,
			.EffectivePort = static_cast<uint16>(EffectivePort),
			.IsReady = IsReady,
			.DataDir = FString(DataDir),
			.ExecutablePath = FString(ExecutablePath),
			.IsValid = IsValid };
	}
	return ZenLockFileData{};
}

bool
ZenLockFileData::IsLockFileLocked(const TCHAR* FileName, bool bAttemptCleanUp)
{
#if PLATFORM_WINDOWS
	if (bAttemptCleanUp)
	{
		IFileManager::Get().Delete(FileName, false, false, true);
	}
	return IFileManager::Get().FileExists(FileName);
#elif PLATFORM_UNIX || PLATFORM_MAC
	TAnsiStringBuilder<256> LockFilePath;
	LockFilePath << FileName;

	if (bAttemptCleanUp)
	{
		int32 Fd = open(LockFilePath.ToString(), O_RDONLY | O_CLOEXEC);
		if (Fd < 0)
		{
			int LastError = errno;
			if (LastError == ENOENT)
			{
				return false;
			}
			UE_LOGF(LogZenServiceState, Warning, "Failed opening lock file '%s' (err: %d)", *LockFilePath, LastError);
			return true;
		}

		int32 LockRet = flock(Fd, LOCK_EX | LOCK_NB);
		if (LockRet < 0)
		{
			int LastError = errno;
			close(Fd);
			if (LastError != EWOULDBLOCK)
			{
				UE_LOGF(LogZenServiceState, Warning, "Failed locking lock file '%s' (err: %d)", *LockFilePath, LastError);
			}
			return true;
		}

		unlink(LockFilePath.ToString());

		flock(Fd, LOCK_UN);
		close(Fd);
	}

	struct stat   Stat;
	int Res = stat(LockFilePath.ToString(), &Stat);
	if (Res == 0)
	{
		return true;
	}
	int LastError = errno;
	if (LastError == ENOENT)
	{
		return false;
	}
	UE_LOGF(LogZenServiceState, Warning, "Failed checking stat of '%s' (err: %d)", *LockFilePath, LastError);
	return true;
#endif
}

ZenLockFileData
ZenLockFileData::ReadCbLockFile(const TCHAR* FileName)
{
#if PLATFORM_WINDOWS
	// Windows specific lock reading path
	// Uses share flags that are unique to windows to allow us to read file contents while the file may be open for write AND delete by another process (zenserver).

	uint32 Access = GENERIC_READ;
	uint32 WinFlags = FILE_SHARE_READ | FILE_SHARE_WRITE | FILE_SHARE_DELETE;
	uint32 Create = OPEN_EXISTING;

	TStringBuilder<MAX_PATH> FullFileNameBuilder;
	FPathViews::ToAbsolutePath(FileName, FullFileNameBuilder);
	for (TCHAR& Char : MakeArrayView(FullFileNameBuilder))
	{
		if (Char == TEXT('/'))
		{
			Char = TEXT('\\');
		}
	}
	if (FullFileNameBuilder.Len() >= MAX_PATH)
	{
		FullFileNameBuilder.Prepend(TEXTVIEW("\\\\?\\"));
	}

	HANDLE Handle = CreateFileW(*FullFileNameBuilder, Access, WinFlags, NULL, Create, FILE_ATTRIBUTE_NORMAL, NULL);
	if (Handle != INVALID_HANDLE_VALUE)
	{
		ON_SCOPE_EXIT{ CloseHandle(Handle); };
		LARGE_INTEGER LI;
		if (GetFileSizeEx(Handle, &LI))
		{
			checkf(LI.QuadPart == LI.u.LowPart, TEXT("Lock file exceeds supported 2GB limit."));
			int32 FileSize32 = LI.u.LowPart;
			if (FileSize32 != 0)
			{
				FUniqueBuffer FileBytes = FUniqueBuffer::Alloc(FileSize32);
				DWORD ReadBytes = 0;
				if (ReadFile(Handle, FileBytes.GetData(), FileSize32, &ReadBytes, NULL) && (ReadBytes == FileSize32))
				{
					return ReadLockData(std::move(FileBytes));
				}
			}
		}
	}
	return {};
#elif PLATFORM_UNIX || PLATFORM_MAC
	TAnsiStringBuilder<256> LockFilePath;
	LockFilePath << FileName;
	int32 Fd = open(LockFilePath.ToString(), O_RDONLY | O_CLOEXEC);
	if (Fd != -1)
	{
		ON_SCOPE_EXIT{ close(Fd); };
		struct stat Stat;
		int Res = fstat(Fd, &Stat);
		if (Res == 0)
		{
			uint64 FileSize = uint64(Stat.st_size);
			checkf(FileSize < 2u * 1024u * 1024u * 1024u, TEXT("Lock file exceeds supported 2GB limit."));
			if (FileSize != 0)
			{
				FUniqueBuffer FileBytes = FUniqueBuffer::Alloc(FileSize);
				if (read(Fd, FileBytes.GetData(), FileSize) == FileSize)
				{
					return ReadLockData(std::move(FileBytes));
				}
			}
		}
	}
	return {};
#endif
}
