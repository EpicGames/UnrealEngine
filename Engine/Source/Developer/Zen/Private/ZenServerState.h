// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/AnsiString.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "Serialization/CompactBinary.h"

class ZenServerState
{
public:
	ZenServerState(bool ReadOnly);
	~ZenServerState();

	struct ZenServerEntry
	{
		// This matches the structure found in the Zen server
		// https://github.com/EpicGames/zen/blob/main/zenutil/include/zenutil/zenserverprocess.h#L91
		//
		std::atomic<uint32> Pid;
		std::atomic<uint16> DesiredListenPort;
		std::atomic<uint16> Flags;
		uint8				SessionId[12];
		std::atomic<uint32> SponsorPids[8];
		std::atomic<uint16> EffectiveListenPort;
		uint8				Padding[10];

		enum class FlagsEnum : uint16
		{
			kShutdownPlease = 1 << 0,
			kIsReady = 1 << 1,
		};

		bool AddSponsorProcess(uint32 PidToAdd);
	};
	static_assert(sizeof(ZenServerEntry) == 64);

	const ZenServerEntry* LookupByDesiredListenPort(int DesiredListenPort) const;
	ZenServerEntry* LookupByDesiredListenPort(int DesiredListenPort);
	const ZenServerEntry* LookupByEffectiveListenPort(int EffectiveListenPort) const;
	ZenServerEntry* LookupByEffectiveListenPort(int EffectiveListenPort);
	const ZenServerEntry* LookupByPid(uint32 Pid) const;

	static bool IsProcessRunning(uint32 Pid);
	static bool Terminate(uint32 Pid);
	static bool FindRunningProcessId(const TCHAR* ExecutablePath, uint32* OutPid);

private:
	const ZenServerEntry* LookupByDesiredListenPortInternal(int DesiredListenPort) const;
	const ZenServerEntry* LookupByEffectiveListenPortInternal(int EffectiveListenPort) const;
	void* m_hMapFile = nullptr;
	ZenServerEntry* m_Data = nullptr;
	int				m_MaxEntryCount = 65536 / sizeof(ZenServerEntry);
	bool			m_IsReadOnly = true;
};

class ZenSharedEvent
{
public:
	ZenSharedEvent() = delete;
	ZenSharedEvent(const ZenSharedEvent&) = delete;
	ZenSharedEvent(ZenSharedEvent&&) = delete;
	ZenSharedEvent& operator =(const ZenSharedEvent&) = delete;
	ZenSharedEvent& operator =(ZenSharedEvent&&) = delete;

	explicit ZenSharedEvent(FStringView EventName);
	~ZenSharedEvent();

	bool Create();
	bool Exists() const;
	bool Open();
	bool Wait(int TimeoutMs = -1);
	bool Set();
	void Close();

	static FString GetShutdownEventName(uint16 EffectiveListenPort);
	static FString GetStartupEventName();

private:
	FString m_EventName;

#if PLATFORM_WINDOWS
	FString GetFullEventName() const;
	void* m_EventHandle = NULL;
#elif PLATFORM_UNIX || PLATFORM_MAC
	FAnsiString GetEventPath() const;
	int m_Fd = -1;
	int m_Semaphore = -1;
	FAnsiString m_EventPath;
#endif
};

struct ZenLockFileData
{
	uint32 ProcessId = 0;
	FCbObjectId SessionId;
	uint16 EffectivePort = 0;
	bool IsReady = false;
	FString DataDir;
	FString ExecutablePath;
	bool IsValid = false;

	static bool IsLockFileLocked(const TCHAR* FileName, bool bAttemptCleanUp = false);
	static ZenLockFileData ReadCbLockFile(const TCHAR* FileName);
};

