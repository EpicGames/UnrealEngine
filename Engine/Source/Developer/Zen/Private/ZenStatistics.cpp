// Copyright Epic Games, Inc. All Rights Reserved.

#include "Experimental/ZenStatistics.h"
#include "Serialization/CompactBinarySerialization.h"

namespace UE::Zen
{

bool LoadFromCompactBinary(FCbFieldView Field, FZenSizeStats& OutValue)
{
	bool bOk = Field.IsObject();
	bOk = LoadFromCompactBinary(Field["disk"], OutValue.Disk) & bOk;
	bOk = LoadFromCompactBinary(Field["memory"], OutValue.Memory) & bOk;
	return bOk;
}

bool LoadFromCompactBinary(FCbFieldView Field, FZenCIDSizeStats& OutValue)
{
	bool bOk = Field.IsObject();
	bOk = LoadFromCompactBinary(Field["tiny"], OutValue.Tiny) & bOk;
	bOk = LoadFromCompactBinary(Field["small"], OutValue.Small) & bOk;
	bOk = LoadFromCompactBinary(Field["large"], OutValue.Large) & bOk;
	bOk = LoadFromCompactBinary(Field["total"], OutValue.Total) & bOk;
	return bOk;
}

bool LoadFromCompactBinary(FCbFieldView Field, FZenCIDStats& OutValue)
{
	bool bOk = Field.IsObject();
	bOk = LoadFromCompactBinary(Field["size"], OutValue.Size) & bOk;
	return bOk;
}

bool LoadFromCompactBinary(FCbFieldView Field, FZenCacheStats::FGeneralStats& OutValue)
{
	bool bOk = Field.IsObject();
	bOk = LoadFromCompactBinary(Field["size"], OutValue.Size) & bOk;
	bOk = LoadFromCompactBinary(Field["hits"], OutValue.Hits) & bOk;
	bOk = LoadFromCompactBinary(Field["misses"], OutValue.Misses) & bOk;
	bOk = LoadFromCompactBinary(Field["writes"], OutValue.Writes) & bOk;
	bOk = LoadFromCompactBinary(Field["hit_ratio"], OutValue.HitRatio) & bOk;
	bOk = LoadFromCompactBinary(Field["cidhits"], OutValue.CidHits) & bOk;
	bOk = LoadFromCompactBinary(Field["cidmisses"], OutValue.CidMisses) & bOk;
	bOk = LoadFromCompactBinary(Field["cidwrites"], OutValue.CidWrites) & bOk;
	bOk = LoadFromCompactBinary(Field["badrequestcount"], OutValue.BadRequestCount) & bOk;
	return bOk;
}

bool LoadFromCompactBinary(FCbFieldView Field, FZenCacheStats::FRequestStats& OutValue)
{
	bool bOk = Field.IsObject();
	bOk = LoadFromCompactBinary(Field["count"], OutValue.Count) & bOk;
	bOk = LoadFromCompactBinary(Field["rate_mean"], OutValue.RateMean) & bOk;
	bOk = LoadFromCompactBinary(Field["t_avg"], OutValue.TAverage) & bOk;
	bOk = LoadFromCompactBinary(Field["t_min"], OutValue.TMin) & bOk;
	bOk = LoadFromCompactBinary(Field["t_max"], OutValue.TMax) & bOk;
	return bOk;
}

bool LoadFromCompactBinary(FCbFieldView Field, FZenCacheStats::FEndPointStats& OutValue)
{
	bool bOk = Field.IsObject();
	bOk = LoadFromCompactBinary(Field["name"], OutValue.Name) & bOk;
	bOk = LoadFromCompactBinary(Field["url"], OutValue.Url) & bOk;
	bOk = LoadFromCompactBinary(Field["state"], OutValue.Health) & bOk;

	FCbFieldView CacheSubfield = Field["cache"];
	bOk = CacheSubfield.IsObject() & bOk;
	if (CacheSubfield.IsObject())
	{
		bOk = LoadFromCompactBinary(CacheSubfield["hit_ratio"], OutValue.HitRatio) & bOk;
		bOk = LoadFromCompactBinary(CacheSubfield["put_bytes"], OutValue.UploadedMB) & bOk;
		OutValue.UploadedMB = OutValue.UploadedMB / 1024.0 / 1024.0;
		bOk = LoadFromCompactBinary(CacheSubfield["get_bytes"], OutValue.DownloadedMB) & bOk;
		OutValue.DownloadedMB = OutValue.DownloadedMB / 1024.0 / 1024.0;
		bOk = LoadFromCompactBinary(CacheSubfield["error_count"], OutValue.ErrorCount) & bOk;
	}
	return bOk;
}

bool LoadFromCompactBinary(FCbFieldView Field, FZenCacheStats::FUpstreamStats& OutValue)
{
	bool bOk = Field.IsObject();
	bOk = LoadFromCompactBinary(Field["reading"], OutValue.Reading) & bOk;
	bOk = LoadFromCompactBinary(Field["writing"], OutValue.Writing) & bOk;
	bOk = LoadFromCompactBinary(Field["worker_threads"], OutValue.WorkerThreads) & bOk;
	bOk = LoadFromCompactBinary(Field["queue_count"], OutValue.QueueCount) & bOk;
	OutValue.TotalUploadedMB = 0.0;
	OutValue.TotalDownloadedMB = 0.0;

	FCbFieldView EndpointsSubfield = Field["endpoints"];
	if (!EndpointsSubfield.IsNull())
	{
		bOk = EndpointsSubfield.IsArray() & bOk;

		for (FCbFieldView EndpointFieldView : EndpointsSubfield)
		{
			FZenCacheStats::FEndPointStats& NewEndPoint = OutValue.EndPoint.AddDefaulted_GetRef();
			bOk = LoadFromCompactBinary(EndpointFieldView, NewEndPoint) & bOk;
			OutValue.TotalUploadedMB += NewEndPoint.UploadedMB;
			OutValue.TotalDownloadedMB += NewEndPoint.DownloadedMB;
		}
	}

	return bOk;
}

bool LoadFromCompactBinary(FCbFieldView Field, FZenCacheStats& OutValue)
{
	bool bOk = Field.IsObject();
	bOk = LoadFromCompactBinary(Field["cache"], OutValue.General) & bOk;
	bOk = LoadFromCompactBinary(Field["requests"], OutValue.Request) & bOk;

	FCbFieldView UpstreamFieldView = Field["upstream"];
	if (UpstreamFieldView.IsObject())
	{
		bOk = LoadFromCompactBinary(UpstreamFieldView, OutValue.Upstream) & bOk;
		bOk = LoadFromCompactBinary(Field["upstream_gets"], OutValue.UpstreamRequest) & bOk;
	}
	bOk = LoadFromCompactBinary(Field["cid"], OutValue.CID) & bOk;
	return bOk;
}

bool LoadFromCompactBinary(FCbFieldView Field, FZenProjectStats::FReadWriteDeleteStats& OutValue)
{
	bool bOk = Field.IsObject();
	bOk = LoadFromCompactBinary(Field["readcount"], OutValue.ReadCount) & bOk;
	bOk = LoadFromCompactBinary(Field["writecount"], OutValue.WriteCount) & bOk;
	bOk = LoadFromCompactBinary(Field["deletecount"], OutValue.DeleteCount) & bOk;
	return bOk;
}

bool LoadFromCompactBinary(FCbFieldView Field, FZenProjectStats::FHitMissWriteStats& OutValue)
{
	bool bOk = Field.IsObject();
	bOk = LoadFromCompactBinary(Field["hitcount"], OutValue.HitCount) & bOk;
	bOk = LoadFromCompactBinary(Field["misscount"], OutValue.MissCount) & bOk;
	bOk = LoadFromCompactBinary(Field["writecount"], OutValue.WriteCount) & bOk;
	return bOk;
}

bool LoadFromCompactBinary(FCbFieldView Field, FZenProjectStats::FGeneralStats& OutValue)
{
	bool bOk = Field.IsObject();
	bOk = LoadFromCompactBinary(Field["size"], OutValue.Size) & bOk;
	bOk = LoadFromCompactBinary(Field["project"], OutValue.Project) & bOk;
	bOk = LoadFromCompactBinary(Field["oplog"], OutValue.Oplog) & bOk;
	bOk = LoadFromCompactBinary(Field["op"], OutValue.Op) & bOk;
	bOk = LoadFromCompactBinary(Field["chunk"], OutValue.Chunk) & bOk;
	bOk = LoadFromCompactBinary(Field["requestcount"], OutValue.RequestCount) & bOk;
	bOk = LoadFromCompactBinary(Field["badrequestcount"], OutValue.BadRequestCount) & bOk;
	return bOk;
}

bool LoadFromCompactBinary(FCbFieldView Field, FZenProjectStats& OutValue)
{
	bool bOk = Field.IsObject();
	bOk = LoadFromCompactBinary(Field["store"], OutValue.General) & bOk;
	bOk = LoadFromCompactBinary(Field["cid"], OutValue.CID) & bOk;
	return bOk;
}

} // UE::Zen
