// Copyright Epic Games, Inc. All Rights Reserved.
#include "ZenVersion.h"
#include "Containers/StringFwd.h"
#include "Misc/StringBuilder.h"

namespace UE::Zen
{

void FZenVersion::Reset()
{
	MajorVersion = 0;
	MinorVersion = 0;
	PatchVersion = 0;
	Details.Empty();
}

bool FZenVersion::TryParse(const TCHAR* InString)
{
	Reset();

	if (InString == nullptr)
	{
		return false;
	}

	const TCHAR* CurrentSegment = InString;
	uint32 CurrentSegmentIndex;
	for (CurrentSegmentIndex = 0; CurrentSegmentIndex < 3; ++CurrentSegmentIndex)
	{
		TCHAR* End = nullptr;
		uint64 ParsedValue = FCString::Strtoui64(CurrentSegment, &End, 10);

		if (End == CurrentSegment)
		{
			// A segment with zero parsable characters is an error
			break;
		}

		switch (CurrentSegmentIndex)
		{
		case 0:
			MajorVersion = (uint32)ParsedValue;
			break;
		case 1:
			MinorVersion = (uint32)ParsedValue;
			break;
		case 2:
			PatchVersion = (uint32)ParsedValue;
			break;
		}

		if (*End != TCHAR('.'))
		{
			CurrentSegment = End;
			if (*End == TCHAR('-'))
			{
				Details = End + 1;
				CurrentSegment = End + 1 + Details.Len();
			}
			++CurrentSegmentIndex;
			break;
		}
		CurrentSegment = End + 1;
	}

	// Successful parsing is determined by whether we managed to parse at least 1 numeric segment
	// and we've parsed to the end of the given string.  Details are optional.
	return (CurrentSegmentIndex > 0) && (*CurrentSegment == TEXT('\0'));
}

FString FZenVersion::ToString(bool bDetailed) const
{
	TStringBuilder<64> Builder;
	Builder << MajorVersion;
	Builder.AppendChar(TEXT('.'));
	Builder << MinorVersion;
	Builder.AppendChar(TEXT('.'));
	Builder << PatchVersion;
	if (bDetailed && !Details.IsEmpty())
	{
		Builder.AppendChar(TEXT('-'));
		Builder << Details;
	}
	return FString(Builder);
}

bool FZenVersion::operator<(FZenVersion& Other) const
{
	if (MajorVersion < Other.MajorVersion)
	{
		return true;
	}
	else if (MajorVersion == Other.MajorVersion)
	{
		if (MinorVersion < Other.MinorVersion)
		{
			return true;
		}
		else if (MinorVersion == Other.MinorVersion)
		{
			if (PatchVersion < Other.PatchVersion)
			{
				return true;
			}
			else if (PatchVersion == Other.PatchVersion)
			{
				return Details < Other.Details;
			}
		}
	}
	return false;
}

FZenVersion::operator bool() const
{
	return 	MajorVersion != 0 ||
		MinorVersion != 0 ||
		PatchVersion != 0 ||
		!Details.IsEmpty();
}

} // namespace UE::Zen
