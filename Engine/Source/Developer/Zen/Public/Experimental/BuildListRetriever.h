// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Experimental/BuildServerInterface.h"
#include "ZenBuildUtils.h"
#include "Internationalization/Text.h"
#include "Internationalization/Regex.h"
#include "Misc/DateTime.h"

#define UE_API ZEN_API

namespace UE::Zen::Build
{
/**
 * Helper class to query a list of builds for a project of a specified build type in a P4 stream.
 * Call ConnectToBuildService() first to establish connection, before calling QueryBuilds() to retrieve the builds, build query is asynchronous.
 */
class FBuildListRetriever : public TSharedFromThis<FBuildListRetriever>
{
public:
	struct FBuildGroup : public TSharedFromThis<FBuildGroup>
	{
		FString Namespace;
		FString DisplayName;
		FString CommitIdentifier;
		int32 Changelist;
		FDateTime CreatedAt;
		FString Category;
		FString Job;
		FString Suffix;
		TMap<FString, UE::Zen::Build::FBuildServiceInstance::FBuildRecord> NamedArtifacts;
		bool bIsPreflight;
	};


	UE_API FBuildListRetriever();
	typedef TMap<FString, TArray<int32>> FPlatformBuildsMap;
	typedef TUniqueFunction<void()> FOnReadyForQuery;
	typedef TUniqueFunction<void(const FPlatformBuildsMap&)> FOnQueriesComplete;
	typedef TUniqueFunction<void(const TArrayView<TSharedRef<FBuildGroup>>&)> FOnGroupQueriesComplete;
	UE_API void ConnectToBuildService();
	UE_API bool IsConnected() const;
	UE_API bool QueryBuilds(const FString& InProjectName, const FString& InBuildType, const FString& InStream, FOnQueriesComplete&& InOnQueriesCompleteCallback);
	UE_API bool QueryBuildGroups(const FString& InProjectName, const FString& InBuildType, const FString& InStream, FOnGroupQueriesComplete&& InOnQueriesCompleteCallback);
	UE_API bool IsReadyForQuery() const;
	UE_API void SetOnReadyForQuery( FOnReadyForQuery&& InOnReadyForQuery );
	UE_API void Refresh();

	UE_API TSharedRef<FBuildServiceInstance> GetServiceInstance() const;


	enum class EBuildType
	{
		Oplog,
		StagedBuild,
		PackagedBuild,
		EditorPreCompiledBinary,
		EditorInstalledBuild,
		Unknown,
		Count
	};
	struct FBuildType : TSharedFromThis<FBuildType>
	{
		EBuildType Type;
		FText DisplayName;
		int Order;
	};

	UE_API TMap<FString,TSharedRef<FBuildType, ESPMode::ThreadSafe>> GenerateBuildTypesMap() const;


	struct FBucketInfo
	{
		FString Project;
		FString BuildType;
		FString Stream;
		FString Platform;
	};
	static UE_API bool GetBucketInfo( const FString& BucketId, FBucketInfo& OutBucketInfo );

private:
	void OnNamespacesAndBucketsRefreshed();
	void IssueBuildQueries(bool bIsGroupQuery);
	void ExtractBuildsFromQueryResult(FListBuildsState& ListBuildsState);
	TArray<TSharedRef<FBuildGroup>> ExtractBuildGroupsFromQueryResult(FListBuildsState& ListBuildsState);
	TSharedRef<FBuildType, ESPMode::ThreadSafe> MakeBuildType(const FString& InBuildTypeName) const;


	FString ProjectName;
	FString BuildType;
	FString Stream;

	mutable TSharedPtr<FBuildServiceInstance> ServiceInstance;
	FPlatformBuildsMap PerPlatformBuilds;
	FOnReadyForQuery OnReadyForQuery;
	FOnQueriesComplete OnQueriesCompleteCallback;
	FOnGroupQueriesComplete OnGroupQueriesCompleteCallback;
	bool ReadyToIssueBuildQuery = false; // Should only be accessed from the game thread
	bool bConnected = false;
};

} // namespace UE::Zen::Build

#undef UE_API
