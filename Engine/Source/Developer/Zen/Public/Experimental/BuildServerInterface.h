// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/AnsiString.h"
#include "Containers/CircularBuffer.h"
#include "Containers/Map.h"
#include "Containers/MpscQueue.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Containers/Ticker.h"
#include "Containers/UnrealString.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Thread.h"
#include "Misc/MonotonicTime.h"
#include "Serialization/CompactBinary.h"
#include "Templates/Function.h"
#include "Templates/PimplPtr.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include <atomic>

#define UE_API ZEN_API


enum class EDesktopLoginInteractionLevel : uint8;

namespace UE { class IHttpClient; }
namespace UE::Zen { class FScopeZenService; }
namespace UE::Zen::Build::Private { class FAccessToken; }
namespace UE::Zen::Build::Private { struct FBuildTransferState; }

namespace UE::Zen::Build
{

struct FServiceSettings
{
public:
	UE_API bool ReadFromConfig();
	UE_API bool ReadFromURL(FStringView InstanceURL);

	inline const FString& GetHost() const { return Host; }
	inline const FString& GetAuthScheme() const { return AuthScheme; }
	inline const FString& GetOAuthProviderIdentifier() const { return OAuthProviderIdentifier; }
private:
	FString Host;
	FString AuthScheme;
	FString OAuthProviderIdentifier;
};

enum class EBuildTransferRequestFlags
{
	None = 0,
	Clean = 1 << 0,
	Force = 1 << 1,
	Scavenge = 1 << 2,
	Verify = 1 << 3,
	BoostWorkerCount = 1 << 4,
	BoostWorkerMemory = 1 << 5,
	BoostWorkers = BoostWorkerCount | BoostWorkerMemory,
	Append = 1 << 6,
};
ENUM_CLASS_FLAGS(EBuildTransferRequestFlags);

class FBuildServiceInstance
{
public:

	enum class EBuildTransferStatus
	{
		Invalid,
		Queued,
		Active,
		Failed,
		Canceled,
		Succeeded
	};

	enum class EConnectionState
	{
		NotStarted,
		ConnectionInProgress,
		ConnectionSucceeded,
		ConnectionFailed
	};

	enum class EConnectionFailureReason
	{
		MissingHost,
		FailedToResolveHost,
		FailedToAcquireAccessToken,
		UnexpectedState,
		Unknown,
		None
	};

	struct FBuildRecord
	{
		FCbObjectId BuildId;
		FString BucketId;
		FCbObject Metadata;

		UE_API FString GetCommitIdentifier() const;
		UE_API FString GetCookPlatform() const;
	};

	struct FBuildFile
	{
		FString Name;
		uint64 RawSize = 0;
		uint32 Attributes = 0;
	};

	struct FBuildPart
	{
		FCbObjectId Id;
		TArray<FBuildFile> Files;
	};

	struct FBuildTransfer
	{
	public:
		enum class EType
		{
			Files,
			Oplog,
			Count
		};

		UE_API EType GetType() const;
		UE_API FString GetDestination() const;

		UE_API FString GetDescription() const;
		UE_API FString GetRecentOutput() const;
		UE_API EBuildTransferStatus GetStatus() const;
		UE_API FString GetLogFilename() const;

		UE_API FMonotonicTimeSpan GetTransferDuration() const;

		UE_API bool GetCurrentProgress(FString& OutLabel, FString& OutDetail, float& OutPercent) const;
		UE_API bool GetOverallProgress(FString& OutLabel, FString& OutDetail, float& OutPercent) const;

		UE_API void RequestCancel();
	private:
		TSharedPtr<Private::FBuildTransferState> State;
		friend class UE::Zen::Build::FBuildServiceInstance;
	};

	typedef TUniqueFunction<void(EConnectionState, EConnectionFailureReason)> FOnConnectionComplete;
	typedef TUniqueFunction<void()> FOnRefreshNamespacesAndBucketsComplete;
	typedef TUniqueFunction<void(TArray<FBuildRecord>&& ListBuildRecords)> FOnListBuildsComplete;
	typedef TUniqueFunction<void(TMap<FString, FBuildPart>&& BuildContents)> FOnListBuildContentsComplete;


	UE_API FBuildServiceInstance();
	UE_API FBuildServiceInstance(FStringView InstanceURL);

	UE_API virtual ~FBuildServiceInstance();

	const FServiceSettings& GetSettings() const
	{
		return Settings;
	}

	UE_API void Connect(bool bAllowInteractive, FOnConnectionComplete&& OnConnectionComplete = {});
	UE_API void RefreshNamespacesAndBuckets(FOnRefreshNamespacesAndBucketsComplete&& InOnRefreshNamespacesAndBucketsComplete = {});
	[[nodiscard]] UE_API EConnectionState GetConnectionState() const;
	[[nodiscard]] UE_API FAnsiStringView GetEffectiveDomain() const;

	[[nodiscard]] TMulticastDelegateRegistration<void()>& OnRefreshNamespacesAndBucketsComplete() { return RefreshNamespacesAndBucketsComplete; }

	inline const TMultiMap<FString, FString>& GetNamespacesAndBuckets() const { return NamespacesAndBuckets; }

	UE_API void ListBuilds(FStringView Namespace, FStringView Bucket, FCbObject&& Query, FOnListBuildsComplete&& InOnListBuildsComplete);
	UE_API void ListBuildsAcrossBuckets(FStringView Namespace, FStringView BucketRegex, FCbObject&& Query, FOnListBuildsComplete&& InOnListBuildsComplete);
	UE_API void ListBuildContents(FStringView Namespace, FStringView Bucket, FStringView BuildId, FOnListBuildContentsComplete&& InOnListBuildContentsComplete);

	UE_API FBuildTransfer StartBuildTransfer(
		const FCbObjectId& BuildId,
		FStringView Name,
		FStringView DestinationFolder,
		FStringView Namespace,
		FStringView Bucket,
		FStringView IncludeFilter = FStringView(),
		FStringView ExcludeFilter = FStringView(),
		FString&& DownloadSpecJSONContents = FString(),
		TArray<FString>&& PartIds = TArray<FString>(),
		TArray<FString>&& PartNames = TArray<FString>(),
		FStringView HostOverride = FStringView(),
		EBuildTransferRequestFlags RequestFlags = EBuildTransferRequestFlags::Scavenge
		);
	UE_API FBuildTransfer StartOplogBuildTransfer(
		const FCbObjectId& BuildId,
		FStringView Name,
		FStringView DestinationProjectId,
		FStringView DestinationOplogId,
		FStringView RootDir,
		FStringView EngineDir,
		FStringView ProjectFilePath,
		FStringView Namespace,
		FStringView Bucket,
		FName TargetPlatformName,
		FStringView HostOverride = FStringView(),
		EBuildTransferRequestFlags RequestFlags = EBuildTransferRequestFlags::Clean
		);
	UE_API FBuildTransfer RepeatBuildTransfer(FBuildTransfer ExistingTransfer);
private:
	typedef TUniqueFunction<void(FString& CommandLineArgs)> FPreZenUtilityInvocation;
	typedef TUniqueFunction<void(bool)> FOnZenUtilityInvocationComplete;

	void Initialize();
	bool AcquireAccessToken(EDesktopLoginInteractionLevel InteractionLevel);
	void SetAccessTokenAndUnlock(FScopeLock &Lock, FStringView Token, double RefreshDelay = 0.0);
	[[nodiscard]] FString GetAccessToken() const;
	FString AddZenUtilityBuildServerArguments(const TCHAR* InCommandLineArgs) const;
	bool InvokeZenUtilitySync(FStringView CommandlineArgs, FString* Output = nullptr);
	void InvokeZenUtility(FStringView CommandlineArgs, FPreZenUtilityInvocation&& PreInvocation, FOnZenUtilityInvocationComplete&& OnComplete);
	void CallOnRefreshNamespacesAndBucketsComplete(FOnRefreshNamespacesAndBucketsComplete&& InOnRefreshNamespacesAndBucketsComplete);
	
	void RecordAcquireAccessTokenEvent(bool Succeeded, const FString& MessageString = {});
	void RecordConnectionEvent(bool Succeeded, const FString& MessageString = {});
	void RecordZenServerEvent(bool Succeeded, const FString& MessageString = {});
	void RecordTransferStateEvent(TSharedRef<Private::FBuildTransferState> State);

	void KickBuildTransferThread();
	void BuildTransferThreadLoop();

	TMulticastDelegate<void()> RefreshNamespacesAndBucketsComplete;
	std::atomic<EConnectionState> ConnectionState = EConnectionState::NotStarted;

	FCriticalSection AccessCs;
	TUniquePtr<Private::FAccessToken> Access;
	FTSTicker::FDelegateHandle RefreshAccessTokenHandle;
	double RefreshAccessTokenTime = 0.0;
	uint32 LoginAttempts = 0;
	uint32 FailedLoginAttempts = 0;
	uint32 InteractiveLoginAttempts = 0;

	FRWLock NamespacesAndBucketsLock;
	TMultiMap<FString, FString> NamespacesAndBuckets;

	TPimplPtr<FScopeZenService> ZenService;

	TMpscQueue<TSharedRef<Private::FBuildTransferState>> BuildTransferThreadStates;
	FThread BuildTransferThread;
	std::atomic<bool> bBuildTransferThreadStarting = false;
	std::atomic<bool> bBuildTransferThreadStopping = false;

	FServiceSettings Settings;
	FAnsiString EffectiveDomain;
};

bool LoadFromCompactBinary(FCbFieldView Field, FBuildServiceInstance::FBuildRecord& OutBuildRecord);

} // namespace UE::Zen::Build

#undef UE_API
