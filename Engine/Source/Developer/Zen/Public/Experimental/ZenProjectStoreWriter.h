// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Serialization/Archive.h"
#include "Templates/UniquePtr.h"

#define UE_API ZEN_API

namespace UE::Zen
{

class FZenProjectStoreWriter
{
public:
	UE_API FZenProjectStoreWriter(const TCHAR* TargetFileName);
	~FZenProjectStoreWriter() = default;

	UE_API bool Write(FStringView ServiceHostName, uint16 ServicePort, const FString& HostAuthJson, bool bIsRunningLocally, FStringView ProjectId, FStringView OplogId, FName TargetPlatformName);
private:
	TUniquePtr<FArchive> OplogMarker;
};

} // namespace UE::Zen

#undef UE_API
