// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "Async/Mutex.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Dom/JsonObject.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Misc/Optional.h"
#include "Misc/TVariant.h"
#include "Policies/PrettyJsonPrintPolicy.h"
#include "Serialization/JsonWriter.h"
#include "Templates/PimplPtr.h"
#include "Templates/UniquePtr.h"
#include "ZenStatistics.h"

#define UE_API ZEN_API

struct FAnalyticsEventAttribute;
class FCbFieldView;

namespace UE::Zen
{

enum class ERemoteNetworkService : uint8
{
	None,
	Unsecured,
	GeneratedStaticKey,
};

struct FServiceConnectSettings
{
	FString HostName;
	uint16 Port = 8558;
};

struct FServicePluginSettings
{
	FString Name;
	FString AbsPath;
	TMap<FName, FString> Options;
	bool bNeedsInstall = false;

	UE_API bool ReadFromConfig(const FString& InPluginName);
	UE_API bool ReadFromCompactBinary(FCbFieldView Field);
	UE_API void WriteToCompactBinary(FCbWriter& Writer) const;
};

struct FServiceAutoLaunchSettings
{
	FString DataPath;
	FString ExtraArgs;
	uint16 DesiredPort = 8558;
	TArray<FServicePluginSettings> Plugins;
	bool bShowConsole = false;
	bool bIsDefaultDataPath = false;
	bool bLimitProcessLifetime = false;
	ERemoteNetworkService RemoteNetworkService = ERemoteNetworkService::None;
	bool bSendUnattendedBugReports = false;
	bool bIsDefaultSharedRunContext = true;
	enum class EInstallMode
	{
		Copy,
		Link
	} InstallMode = EInstallMode::Copy;
};

struct FSystemServiceSettings
{
	FString InstallPath;
	FString DataPath;
	FString ExtraArgs;
	uint16 DesiredPort = 8558;
	bool bSendUnattendedBugReports = false;
	ERemoteNetworkService RemoteNetworkService = ERemoteNetworkService::None;
};

struct FServiceSettings
{
	bool bSystemService = false;
	FSystemServiceSettings SystemServiceSettings;
	TVariant<FServiceAutoLaunchSettings, FServiceConnectSettings> SettingsVariant;

	inline bool IsSystemService() const { return bSystemService; }
	inline bool IsAutoLaunch() const { return IsSystemService() || SettingsVariant.IsType<FServiceAutoLaunchSettings>(); }
	inline bool IsConnectExisting() const { return !IsSystemService() && SettingsVariant.IsType<FServiceConnectSettings>(); }
	inline bool IsRequirePlugins() const { return IsAutoLaunch() && !SettingsVariant.Get<FServiceAutoLaunchSettings>().Plugins.IsEmpty(); }
	inline ERemoteNetworkService GetRemoteNetworkService() const
	{
		if (bSystemService) return SystemServiceSettings.RemoteNetworkService;
		if (IsAutoLaunch()) return SettingsVariant.Get<FServiceAutoLaunchSettings>().RemoteNetworkService;
		return ERemoteNetworkService::None;
	}

	UE_API bool ReadFromConfig();
	UE_API bool ReadFromCompactBinary(FCbFieldView Field);
	UE_API bool ReadFromURL(FStringView InstanceURL);

	UE_API void WriteToCompactBinary(FCbWriter& Writer) const;

private:
	bool TryApplyAutoLaunchOverride();
};

struct FZenWorkspaces
{
	struct Share
	{
		FString Id;
		FString Dir;
		FString Alias;
	};

	struct Workspace
	{
		FString Id;
		FString BaseDir;
		bool bDynamicShare;

		TArray<Share> WorkspaceShares;
	};

	TArray<Workspace> ZenWorkspaces;
	bool bIsValid = false;

	bool operator==(const FZenWorkspaces& Other) const
	{
		if (ZenWorkspaces.Num() != Other.ZenWorkspaces.Num())
		{
			return false;
		}

		for (int32 Idx = 0; Idx < ZenWorkspaces.Num(); ++Idx)
		{
			const Workspace& CurrentWorkspace = ZenWorkspaces[Idx];
			const Workspace& OtherWorkspace = Other.ZenWorkspaces[Idx];

			if (CurrentWorkspace.WorkspaceShares.Num() != OtherWorkspace.WorkspaceShares.Num())
			{
				return false;
			}

			if (CurrentWorkspace.Id != OtherWorkspace.Id ||
				CurrentWorkspace.BaseDir != OtherWorkspace.BaseDir ||
				CurrentWorkspace.bDynamicShare != OtherWorkspace.bDynamicShare)
			{
				return false;
			}

			for (int32 ShareIdx = 0; ShareIdx < CurrentWorkspace.WorkspaceShares.Num(); ++ShareIdx)
			{
				const Share& CurrentShare = CurrentWorkspace.WorkspaceShares[ShareIdx];
				const Share& OtherShare = OtherWorkspace.WorkspaceShares[ShareIdx];

				if (CurrentShare.Id != OtherShare.Id ||
					CurrentShare.Dir != OtherShare.Dir ||
					CurrentShare.Alias != OtherShare.Alias)
				{
					return false;
				}
			}
		}

		return true;
	}
};

}

namespace UE::Zen
{

namespace Private
{
	UE_API bool IsLocalAutoLaunched(FStringView InstanceURL);
	UE_API bool GetLocalDataCachePathOverride(FString& OutDataPath);

	// For testing purposes only.
	//
	// This is a bit of a hack, to avoid issues in unit tests where we attempt to remove a copied binary when
	// testing install in link mode, or vice versa. During unit tests, we need to run those installations sandboxed
	// to their own install directory so as not to interfere with an actual running zenserver that is being
	// used outside of the unit tests.
	UE_API FString GetLocalInstallPathOverride();
	UE_API void SetLocalInstallPathOverride(const FString& Path);
}

class FZenServiceInstance;
class FZenLocalServiceRunContext;

UE_API bool TryGetLocalServiceRunContext(FZenLocalServiceRunContext& OutContext);

class FZenLocalServiceRunContext
{
public:
	FString GetExecutable() const { return Executable; }
	FString GetCommandlineArguments() const { return CommandlineArguments; }
	FString GetWorkingDirectory() const { return WorkingDirectory; }
	FString GetDataPath() const { return DataPath; }
	bool GetShowConsole() const { return bShowConsole; }
	bool GetLimitProcessLifetime() const { return bLimitProcessLifetime; }

	friend inline bool operator==(FZenLocalServiceRunContext LHS, FZenLocalServiceRunContext RHS)
	{
		return (LHS.Executable == RHS.Executable) && (LHS.CommandlineArguments == RHS.CommandlineArguments) && (LHS.WorkingDirectory == RHS.WorkingDirectory) && (LHS.DataPath == RHS.DataPath) && (LHS.bLimitProcessLifetime == RHS.bLimitProcessLifetime);
	}

private:
	FString Executable;
	FString CommandlineArguments;
	FString WorkingDirectory;
	FString DataPath;
	bool bShowConsole = false;
	bool bLimitProcessLifetime = false;

	bool ReadFromJson(FJsonObject& JsonObject);
	void WriteToJson(TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>& Writer) const;

	bool ReadFromJsonFile(const TCHAR* Filename);
	bool WriteToJsonFile(const TCHAR* Filename) const;

	friend class FZenServiceInstance;
	friend UE_API bool TryGetLocalServiceRunContext(FZenLocalServiceRunContext& OutContext);
};

UE_API bool IsLocalServiceRunning(const TCHAR* DataPath, uint16* OutPort = nullptr);
UE_API bool StartLocalService(const FZenLocalServiceRunContext& Context);
UE_API bool StopLocalService(const TCHAR* DataPath, double MaximumWaitDurationSeconds = 25.0);

UE_API FString GetLocalServiceInstallPath();
UE_API FString GetLocalServiceInstallVersion(bool bDetailed = true);
UE_API FString GetLocalInstallUtilityPath();

UE_API bool GetLocalServiceHostAuth(FString& OutAuthJson);

struct FGCStatus
{
	FString Description;
};

class FZenServiceEndpoint
{
public:
	enum class ESocketType : uint8
	{
		Tcp,
		Unix,
	};

	FZenServiceEndpoint() = default;
	FZenServiceEndpoint(FStringView InName, uint16 InPort);
	ESocketType GetSocketType() const { return SocketType; }
	FStringView GetName() const { return Name; }
	UE_API FStringView GetHostName() const;
	UE_API FString GetURL() const;
	uint16 GetPort() const { return Port; }

private:
	FString Name = TEXT("localhost");
	uint16 Port = 8558;
	ESocketType SocketType = ESocketType::Tcp;
};

/**
 * Type used to declare usage of a Zen server instance whether the shared default instance or a unique non-default instance.
 * Used to help manage launch, and optionally in the future, shutdown a shared default instance.  Use the default constructor
 * to reference the default instance (which may be launched on demand), or use the non-default constructors with a specific
 * URL or HostName/Port pair which is required to pre-exist (will not be auto-launched).
 */
class FScopeZenService
{
public:
	UE_API FScopeZenService();
	UE_API FScopeZenService(FStringView InstanceURL);
	UE_API FScopeZenService(FServiceSettings&& InSettings);
	UE_API ~FScopeZenService();


	const FZenServiceInstance& GetInstance() const { return *ServiceInstance; }
	FZenServiceInstance& GetInstance() { return *ServiceInstance; }

private:
	FZenServiceInstance* ServiceInstance;
	TUniquePtr<UE::Zen::FZenServiceInstance> UniqueNonDefaultInstance;
};

/**
 * Gets the default Zen service instance.  The default instance is configured through ZenServiceInstance INI section.
 * The default instance can (depending on configuration):
 *  - Auto-launched on demand (optionally elevated on Windows)
 *  - Be copied out of the workspace tree before execution
 *  - Shared between multiple tools running concurrently (implemented by launching multiple instances and expecting them to communicate and shutdown as needed)
 *  - Instigate self-shutdown when all processes that requested it have terminated
 *  - Use a subdirectory of the existing local DDC cache path as Zen's data path
 *  - Use an upstream Zen service
 *  - Be overridden by commandline arguments to reference an existing running instance instead of auto-launching a new instance.
 * Note that no assumptions should be made about the hostname/port of the default instance.  Calling code should instead expect
 * that the instance is authoritative over the hostname/port that it will end up using and should query that information from the
 * instance as needed.
 */
UE_API FZenServiceInstance& GetDefaultServiceInstance();
UE_API bool IsDefaultServicePresent();

/**
 * A representation of a Zen service instance.  Generally not accessed directly, but via FScopeZenService.
 */
class FZenServiceInstance
{
public:

	UE_API FZenServiceInstance();
	UE_API FZenServiceInstance(FStringView InstanceURL);
	UE_API FZenServiceInstance(FServiceSettings&& InSettings);
	UE_API ~FZenServiceInstance();

	UE_API const FString GetPath() const;
	inline const FServiceSettings& GetServiceSettings() const { return Settings; }
	UE_API bool IsServiceRunning();
	UE_API bool IsServiceReady();
	bool IsServiceRunningLocally() const { return bIsRunningLocally; }
	const FZenServiceEndpoint& GetEndpoint() const { return Endpoint; }

	enum class ERecoveryMode
	{
		Soft,	// Try to recover if we haven't done so recently and be relaxed if it fails
		Hard	// Try to recover always and put more effort and retries to it
	};

	UE_API bool TryRecovery(ERecoveryMode RecoveryMode);

	UE_API bool GetCacheStats(FZenCacheStats& Stats);
	UE_API bool GetProjectStats(FZenProjectStats& Stats);
	UE_API bool GatherAnalytics(TArray<FAnalyticsEventAttribute>& Attributes);

	UE_API int32 GetWorkspaceCount() const;
	UE_API bool GetWorkspaces(FZenWorkspaces& Workspaces) const;

	UE_API bool GetGCStatus(FGCStatus& Status);
	UE_API bool RequestGC(const bool* OverrideCollectSmallObjects = nullptr, const uint32* OverrideMaxCacheDuration = nullptr);

	UE_API bool AddSponsorProcessIDs(TArrayView<uint32> SponsorProcessIDs);

	static UE_API uint16 GetAutoLaunchedPort();

private:

	void Initialize();
	bool ConditionalUpdateLocalInstall(FServiceAutoLaunchSettings::EInstallMode InstallMode);
	bool WriteLocalPluginsConfig();
	static bool AutoLaunch(const FServiceAutoLaunchSettings& InSettings, const FString& ExecutablePath, const FString& ExecutionContextFilePath, FZenServiceEndpoint& OutEndpoint);
	bool ConditionalUpdateSystemServiceInstall(const FSystemServiceSettings& InSettings);
	bool PingService(uint32_t AttemptCount, bool Silent);

	mutable TFuture<TOptional<FGCStatus>> GCStatusRequest;
	mutable TOptional<FGCStatus> LastGCStatus;
	mutable uint64 LastGCStatusTime = 0;

	mutable TFuture<FZenCacheStats> CacheStatsRequest;
	mutable uint64 LastCacheStatsTime = 0;
	mutable FMutex LastCacheStatsMutex;
	mutable FZenCacheStats LastCacheStats;

	mutable TFuture<FZenProjectStats> ProjectStatsRequest;
	mutable uint64 LastProjectStatsTime = 0;
	mutable FMutex LastProjectStatsMutex;
	mutable FZenProjectStats LastProjectStats;

	mutable TFuture<FZenWorkspaces> WorkspacesRequest;

	FServiceSettings Settings;

	FZenServiceEndpoint Endpoint;
	static uint16 AutoLaunchedPort;
	static uint32 AutoLaunchedPid;
	static const uint32 DefaultSmallOperationConnectTimeoutMS;
	static const uint32 DefaultSmallOperationTimeoutMS;
	bool bHasLaunchedLocal = false;
	bool bHasLaunchedSystemService = false;
	bool bIsRunningLocally = true;
};

} // namespace UE::Zen

#undef UE_API
