// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

#define UE_API ZEN_API

class FCbFieldView;

namespace UE::Zen
{

struct FZenSizeStats
{
	double Disk = 0;
	double Memory = 0;
};

struct FZenCIDSizeStats
{
	int64 Tiny = 0;
	int64 Small = 0;
	int64 Large = 0;
	int64 Total = 0;
};

struct FZenCIDStats
{
	FZenCIDSizeStats Size;
};

struct FZenCacheStats
{
	struct FGeneralStats
	{
		FZenSizeStats Size;
		int64 Hits = 0;
		int64 Misses = 0;
		int64 Writes = 0;
		double HitRatio = 0.0;
		int64 CidHits = 0;
		int64 CidMisses = 0;
		int64 CidWrites = 0;
		int64 BadRequestCount = 0;
	};

	struct FRequestStats
	{
		int64 Count = 0;
		double RateMean = 0.0;
		double TAverage = 0.0;
		double TMin = 0.0;
		double TMax = 0.0;
	};

	struct FEndPointStats
	{
		FString Name;
		FString Url;
		FString Health;
		double HitRatio = 0.0;
		double DownloadedMB = 0;
		double UploadedMB = 0;
		int64 ErrorCount = 0;
	};

	struct FUpstreamStats
	{
		bool Reading = false;
		bool Writing = false;
		int64 WorkerThreads = 0;
		int64 QueueCount = 0;
		double TotalUploadedMB = 0;
		double TotalDownloadedMB = 0;
		TArray<FEndPointStats> EndPoint;
	};

	FGeneralStats General;
	FRequestStats Request;
	FUpstreamStats Upstream;
	FRequestStats UpstreamRequest;
	FZenCIDStats CID;
	bool bIsValid = false;
};

struct FZenProjectStats
{
	struct FReadWriteDeleteStats
	{
		int64 ReadCount;
		int64 WriteCount;
		int64 DeleteCount;
	};
	struct FHitMissWriteStats
	{
		int64 HitCount;
		int64 MissCount;
		int64 WriteCount;
	};
	struct FGeneralStats
	{
		FZenSizeStats Size;
		FReadWriteDeleteStats Project;
		FReadWriteDeleteStats Oplog;
		FHitMissWriteStats Op;
		FHitMissWriteStats Chunk;
		int64 RequestCount;
		int64 BadRequestCount;
	};
	FGeneralStats General;
	FZenCIDStats CID;
	bool bIsValid = false;
};

UE_API bool LoadFromCompactBinary(FCbFieldView Field, FZenSizeStats& OutValue);
UE_API bool LoadFromCompactBinary(FCbFieldView Field, FZenCIDSizeStats& OutValue);
UE_API bool LoadFromCompactBinary(FCbFieldView Field, FZenCIDStats& OutValue);

UE_API bool LoadFromCompactBinary(FCbFieldView Field, FZenCacheStats::FGeneralStats& OutValue);
UE_API bool LoadFromCompactBinary(FCbFieldView Field, FZenCacheStats::FRequestStats& OutValue);
UE_API bool LoadFromCompactBinary(FCbFieldView Field, FZenCacheStats::FEndPointStats& OutValue);
UE_API bool LoadFromCompactBinary(FCbFieldView Field, FZenCacheStats::FUpstreamStats& OutValue);
UE_API bool LoadFromCompactBinary(FCbFieldView Field, FZenCacheStats& OutValue);

UE_API bool LoadFromCompactBinary(FCbFieldView Field, FZenProjectStats::FReadWriteDeleteStats& OutValue);
UE_API bool LoadFromCompactBinary(FCbFieldView Field, FZenProjectStats::FHitMissWriteStats& OutValue);
UE_API bool LoadFromCompactBinary(FCbFieldView Field, FZenProjectStats::FGeneralStats& OutValue);
UE_API bool LoadFromCompactBinary(FCbFieldView Field, FZenProjectStats& OutValue);

} // namespace UE::Zen

#undef UE_API
