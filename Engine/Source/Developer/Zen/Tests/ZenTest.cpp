// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS


#include "TestHarness.h"

#if WITH_EDITORONLY_DATA
#include "Experimental/ZenServerInterface.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/ScopeExit.h"

namespace UE
{

TEST_CASE("Zen::ZenServerInterface", "[Zen][Basic]")
{
	using namespace UE::Zen;
	uint16 DefaultTestPort = 8559;
	uint16 CurrentTestPort = DefaultTestPort + 1;
	FString DefaultArgs = TEXT("--http asio");
	FString DataPathRoot = FPaths::ConvertRelativePathToFull(FPaths::Combine(FPaths::EngineSavedDir(), "ZenServerInterfaceUnitTest"));
	FString DefaultDataPath = FPaths::ConvertRelativePathToFull(FPaths::Combine(DataPathRoot, "Default"));
	IFileManager::Get().DeleteDirectory(*DataPathRoot, false, true);

	Zen::Private::SetLocalInstallPathOverride(*DataPathRoot);
	ON_SCOPE_EXIT
	{
		Zen::Private::SetLocalInstallPathOverride(TEXT(""));
	};

	SECTION("Basic AutoLaunch and Shutdown (copy)")
	{
		for (int Iteration = 0; Iteration < 4; ++Iteration)
		{
			FServiceSettings ZenTestServiceSettings;
			FServiceAutoLaunchSettings& ZenTestAutoLaunchSettings = ZenTestServiceSettings.SettingsVariant.Get<FServiceAutoLaunchSettings>();
			ZenTestAutoLaunchSettings.DataPath = DefaultDataPath;
			ZenTestAutoLaunchSettings.ExtraArgs = DefaultArgs;
			ZenTestAutoLaunchSettings.DesiredPort = DefaultTestPort;
			ZenTestAutoLaunchSettings.InstallMode = FServiceAutoLaunchSettings::EInstallMode::Copy;

			{
				FScopeZenService ScopeZenService(MoveTemp(ZenTestServiceSettings));
				FZenServiceInstance& ZenInstance = ScopeZenService.GetInstance();
				uint16 AutoLaunchedPort = ZenInstance.GetAutoLaunchedPort();
				uint16 DetectedPort = 0;
				CHECK(ZenInstance.IsServiceReady());

				CHECK(IsLocalServiceRunning(*DefaultDataPath, &DetectedPort));
				CHECK(DetectedPort == AutoLaunchedPort);
				CHECK(StopLocalService(*DefaultDataPath));
				CHECK(!IsLocalServiceRunning(*DefaultDataPath));
			}
		}
	}

	SECTION("Basic AutoLaunch and Shutdown (link)")
	{
		for (int Iteration = 0; Iteration < 4; ++Iteration)
		{
			FServiceSettings ZenTestServiceSettings;
			FServiceAutoLaunchSettings& ZenTestAutoLaunchSettings = ZenTestServiceSettings.SettingsVariant.Get<FServiceAutoLaunchSettings>();
			ZenTestAutoLaunchSettings.DataPath = DefaultDataPath;
			ZenTestAutoLaunchSettings.ExtraArgs = DefaultArgs;
			ZenTestAutoLaunchSettings.DesiredPort = DefaultTestPort;
			ZenTestAutoLaunchSettings.InstallMode = FServiceAutoLaunchSettings::EInstallMode::Link;

			{
				FScopeZenService ScopeZenService(MoveTemp(ZenTestServiceSettings));
				FZenServiceInstance& ZenInstance = ScopeZenService.GetInstance();
				uint16 AutoLaunchedPort = ZenInstance.GetAutoLaunchedPort();
				uint16 DetectedPort = 0;
				CHECK(ZenInstance.IsServiceReady());

				CHECK(IsLocalServiceRunning(*DefaultDataPath, &DetectedPort));
				CHECK(DetectedPort == AutoLaunchedPort);
				CHECK(StopLocalService(*DefaultDataPath));
				CHECK(!IsLocalServiceRunning(*DefaultDataPath));
			}
		}
	}

	SECTION("Overlapping AutoLaunch and Shutdown (copy)")
	{
		for (int Iteration = 0; Iteration < 11; ++Iteration)
		{
			FServiceSettings ZenTestServiceSettings;
			FServiceAutoLaunchSettings& ZenTestAutoLaunchSettings = ZenTestServiceSettings.SettingsVariant.Get<FServiceAutoLaunchSettings>();
			ZenTestAutoLaunchSettings.DataPath = DefaultDataPath;
			ZenTestAutoLaunchSettings.ExtraArgs = DefaultArgs;
			ZenTestAutoLaunchSettings.DesiredPort = DefaultTestPort;
			ZenTestAutoLaunchSettings.InstallMode = FServiceAutoLaunchSettings::EInstallMode::Copy;

			{
				FScopeZenService ScopeZenService(MoveTemp(ZenTestServiceSettings));
				FZenServiceInstance& ZenInstance = ScopeZenService.GetInstance();
				uint16 AutoLaunchedPort = ZenInstance.GetAutoLaunchedPort();
				uint16 DetectedPort = 0;
				CHECK(ZenInstance.IsServiceReady());

				CHECK(IsLocalServiceRunning(*DefaultDataPath, &DetectedPort));
				CHECK(DetectedPort == AutoLaunchedPort);
			}
		}
		CHECK(StopLocalService(*DefaultDataPath));
		CHECK(!IsLocalServiceRunning(*DefaultDataPath));
	}

	SECTION("Overlapping AutoLaunch and Shutdown (link)")
	{
		for (int Iteration = 0; Iteration < 11; ++Iteration)
		{
			FServiceSettings ZenTestServiceSettings;
			FServiceAutoLaunchSettings& ZenTestAutoLaunchSettings = ZenTestServiceSettings.SettingsVariant.Get<FServiceAutoLaunchSettings>();
			ZenTestAutoLaunchSettings.DataPath = DefaultDataPath;
			ZenTestAutoLaunchSettings.ExtraArgs = DefaultArgs;
			ZenTestAutoLaunchSettings.DesiredPort = DefaultTestPort;
			ZenTestAutoLaunchSettings.InstallMode = FServiceAutoLaunchSettings::EInstallMode::Link;

			{
				FScopeZenService ScopeZenService(MoveTemp(ZenTestServiceSettings));
				FZenServiceInstance& ZenInstance = ScopeZenService.GetInstance();
				uint16 AutoLaunchedPort = ZenInstance.GetAutoLaunchedPort();
				uint16 DetectedPort = 0;
				CHECK(ZenInstance.IsServiceReady());

				CHECK(IsLocalServiceRunning(*DefaultDataPath, &DetectedPort));
				CHECK(DetectedPort == AutoLaunchedPort);
			}
		}
		CHECK(StopLocalService(*DefaultDataPath));
		CHECK(!IsLocalServiceRunning(*DefaultDataPath));
	}

	SECTION("Overlapping AutoLaunch and Shutdown With DataPath Shared And Differing Args (copy)")
	{
		for (int Iteration = 0; Iteration < 11; ++Iteration)
		{
			FServiceSettings ZenTestServiceSettings;
			FServiceAutoLaunchSettings& ZenTestAutoLaunchSettings = ZenTestServiceSettings.SettingsVariant.Get<FServiceAutoLaunchSettings>();
			ZenTestAutoLaunchSettings.DataPath = DefaultDataPath;
			ZenTestAutoLaunchSettings.ExtraArgs = *WriteToString<128>(DefaultArgs, TEXT(" --gc-interval-seconds "), (Iteration+1)*1000);
			ZenTestAutoLaunchSettings.DesiredPort = DefaultTestPort;
			ZenTestAutoLaunchSettings.InstallMode = FServiceAutoLaunchSettings::EInstallMode::Copy;

			{
				FScopeZenService ScopeZenService(MoveTemp(ZenTestServiceSettings));
				FZenServiceInstance& ZenInstance = ScopeZenService.GetInstance();
				uint16 AutoLaunchedPort = ZenInstance.GetAutoLaunchedPort();
				uint16 DetectedPort = 0;
				CHECK(ZenInstance.IsServiceReady());

				CHECK(IsLocalServiceRunning(*DefaultDataPath, &DetectedPort));
				CHECK(DetectedPort == AutoLaunchedPort);
			}
		}
		CHECK(StopLocalService(*DefaultDataPath));
		CHECK(!IsLocalServiceRunning(*DefaultDataPath));
	}

	SECTION("Overlapping AutoLaunch and Shutdown With DataPath Shared And Differing Args (link)")
	{
		for (int Iteration = 0; Iteration < 11; ++Iteration)
		{
			FServiceSettings ZenTestServiceSettings;
			FServiceAutoLaunchSettings& ZenTestAutoLaunchSettings = ZenTestServiceSettings.SettingsVariant.Get<FServiceAutoLaunchSettings>();
			ZenTestAutoLaunchSettings.DataPath = DefaultDataPath;
			ZenTestAutoLaunchSettings.ExtraArgs = *WriteToString<128>(DefaultArgs, TEXT(" --gc-interval-seconds "), (Iteration + 1) * 1000);
			ZenTestAutoLaunchSettings.DesiredPort = DefaultTestPort;
			ZenTestAutoLaunchSettings.InstallMode = FServiceAutoLaunchSettings::EInstallMode::Link;

			{
				FScopeZenService ScopeZenService(MoveTemp(ZenTestServiceSettings));
				FZenServiceInstance& ZenInstance = ScopeZenService.GetInstance();
				uint16 AutoLaunchedPort = ZenInstance.GetAutoLaunchedPort();
				uint16 DetectedPort = 0;
				CHECK(ZenInstance.IsServiceReady());

				CHECK(IsLocalServiceRunning(*DefaultDataPath, &DetectedPort));
				CHECK(DetectedPort == AutoLaunchedPort);
			}
		}
		CHECK(StopLocalService(*DefaultDataPath));
		CHECK(!IsLocalServiceRunning(*DefaultDataPath));
	}

	SECTION("Overlapping AutoLaunch and Shutdown With DataPath Shared And Differing Args (alternating)")
	{
		for (int Iteration = 0; Iteration < 11; ++Iteration)
		{
			FServiceSettings ZenTestServiceSettings;
			FServiceAutoLaunchSettings& ZenTestAutoLaunchSettings = ZenTestServiceSettings.SettingsVariant.Get<FServiceAutoLaunchSettings>();
			ZenTestAutoLaunchSettings.DataPath = DefaultDataPath;
			ZenTestAutoLaunchSettings.ExtraArgs = *WriteToString<128>(DefaultArgs, TEXT(" --gc-interval-seconds "), (Iteration + 1) * 1000);
			ZenTestAutoLaunchSettings.DesiredPort = DefaultTestPort;
			ZenTestAutoLaunchSettings.InstallMode = ((Iteration & 1) == 0) ? FServiceAutoLaunchSettings::EInstallMode::Copy : FServiceAutoLaunchSettings::EInstallMode::Link;

			{
				FScopeZenService ScopeZenService(MoveTemp(ZenTestServiceSettings));
				FZenServiceInstance& ZenInstance = ScopeZenService.GetInstance();
				uint16 AutoLaunchedPort = ZenInstance.GetAutoLaunchedPort();
				uint16 DetectedPort = 0;
				CHECK(ZenInstance.IsServiceReady());

				CHECK(IsLocalServiceRunning(*DefaultDataPath, &DetectedPort));
				CHECK(DetectedPort == AutoLaunchedPort);
			}
		}
		CHECK(StopLocalService(*DefaultDataPath));
		CHECK(!IsLocalServiceRunning(*DefaultDataPath));
	}
}

} // UE
#endif // WITH_EDITORONLY_DATA

#endif // WITH_LOW_LEVEL_TESTS
