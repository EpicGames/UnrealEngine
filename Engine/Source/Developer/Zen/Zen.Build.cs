// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

[SupportedPlatformGroups("Desktop")]
public class Zen : ModuleRules
{
	public Zen(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;

		PrivateDependencyModuleNames.AddRange(new string[] { "Core", "Sockets", "SSL", "Json", "DevHttp",  "StudioTelemetry", "HTTP" });
		PrivateIncludePathModuleNames.AddRange(new string[] { "DesktopPlatform", "Analytics" });
		
		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) && Target.WindowsPlatform.bUseXCurl)
		{
			PublicDependencyModuleNames.Add("XCurl");
		}
		else
		{		
		AddEngineThirdPartyPrivateStaticDependencies(Target, "libcurl");
		}
		AddEngineThirdPartyPrivateStaticDependencies(Target, "OpenSSL");

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		PrivateDefinitions.Add("WITH_CURL=1");
	}
}
